[CoreRedirects]
; AnimNextChooser plugin -> UAFChooser plugin
+PackageRedirects=(OldName="/Script/AnimNextControlRig", NewName="/Script/UAFControlRig")

+StructRedirects=(OldName="/Script/AnimNextControlRig.ControlRigEventName", NewName="/Script/UAFControlRig.ControlRigEventName")
+StructRedirects=(OldName="/Script/AnimNextControlRig.ControlRigExposedProperty", NewName="/Script/UAFControlRig.ControlRigExposedProperty")
+StructRedirects=(OldName="/Script/AnimNextControlRig.ControlRigTraitSharedData", NewName="/Script/UAFControlRig.ControlRigTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextControlRig.AnimNextControlRigTask", NewName="/Script/UAFControlRig.AnimNextControlRigTask")

