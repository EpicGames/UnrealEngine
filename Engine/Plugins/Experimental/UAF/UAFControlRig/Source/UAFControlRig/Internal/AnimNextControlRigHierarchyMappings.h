// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ReferencePose.h"

class UControlRig;
class URigHierarchy;
class UNodeMappingContainer;
class USkeletalMeshComponent;
struct FBoneContainer;
struct FBoneReference;
struct FPoseContext;
struct FControlRigIOSettings;
struct FCompactPose;

namespace UE::UAF
{
struct FKeyframeState;
}

namespace UE::UAF::ControlRig
{

class FAnimNextControlRigPoseAdapter;

struct UAFCONTROLRIG_API FAnimNextControlRigHierarchyMappings
{
	FAnimNextControlRigHierarchyMappings() = default;

	void InitializeInstance();

	void LinkToHierarchy(URigHierarchy* InHierarchy);

	bool CanExecute() const
	{
		return PoseAdapter.IsValid();
	}

	void ResetRefPoseSetterHash()
	{
		RefPoseSetterHash.Reset();
	}

	void UpdateControlRigRefPoseIfNeeded(UControlRig* ControlRig
		, const UObject* InstanceObject
		, const USkeletalMeshComponent* SkeletalMeshComponent
		, const UE::UAF::FReferencePose& InRefPose
		, bool bInSetRefPoseFromSkeleton
		, bool bIncludePoseInHash);

	void UpdateInputOutputMappingIfRequired(UControlRig* InControlRig
		, URigHierarchy* InHierarchy
		, const UE::UAF::FReferencePose& InRefPose
		, int32 InCurrentLOD
		, const TArray<FBoneReference>& InInputBonesToTransfer
		, const TArray<FBoneReference>& InOutputBonesToTransfer
		, const TWeakObjectPtr<UNodeMappingContainer>& InNodeMappingContainer
		, bool bInTransferPoseInGlobalSpace
		, bool bResetInputPoseToInitial);

	void UpdateInput(UControlRig* ControlRig
		, UE::UAF::FKeyframeState& InOutput
		, const FControlRigIOSettings& InInputSettings
		, const FControlRigIOSettings& InOutputSettings
		, const TWeakObjectPtr<UNodeMappingContainer>& InNodeMappingContainer
		, bool bInExecute
		, bool bInTransferInputPose
		, bool bInResetInputPoseToInitial
		, bool bInTransferPoseInGlobalSpace
		, bool bInTransferInputCurves);

	void UpdateOutput(UControlRig* ControlRig
		, UE::UAF::FKeyframeState& InOutput
		, const FControlRigIOSettings& InOutputSettings
		, const TWeakObjectPtr<UNodeMappingContainer>& InNodeMappingContainer
		, bool bInExecute
		, bool bInTransferPoseInGlobalSpace);

	const TArray<TPair<uint16, uint16>>& GetControlRigBoneInputMappingByIndex() const
	{
		return ControlRigBoneInputMappingByIndex;
	}

	TArray<TPair<uint16, uint16>>& GetControlRigBoneOutputMappingByIndex()
	{
		return ControlRigBoneOutputMappingByIndex;
	}

	const TMap<FName, uint16>& GetControlRigBoneInputMappingByName() const
	{
		return ControlRigBoneInputMappingByName;
	}

	TMap<FName, uint16>& GetControlRigBoneOutputMappingByName()
	{
		return ControlRigBoneOutputMappingByName;
	}

	bool CheckPoseAdapter() const;

	bool IsUpdateToDate(const URigHierarchy* InHierarchy) const;

	void PerformUpdateToDate(UControlRig* ControlRig
		, URigHierarchy* InHierarchy
		, const UE::UAF::FReferencePose& InRefPose
		, int32 InCurrentLOD
		, const TWeakObjectPtr<UNodeMappingContainer>& InNodeMappingContainer
		, bool bInTransferPoseInGlobalSpace
		, bool bInResetInputPoseToInitial);

protected:
	/** Complete mapping from skeleton to control rig bone index */
	TArray<TPair<uint16, uint16>> ControlRigBoneInputMappingByIndex;
	TArray<TPair<uint16, uint16>> ControlRigBoneOutputMappingByIndex;

	/** Complete mapping from skeleton to curve name */
	TArray<TPair<uint16, FName>> ControlRigCurveMappingByIndex;

	/** Rig Hierarchy bone name to required array index mapping */
	TMap<FName, uint16> ControlRigBoneInputMappingByName;
	TMap<FName, uint16> ControlRigBoneOutputMappingByName;

	/** Rig Curve name to Curve mapping */
	TMap<FName, FName> ControlRigCurveMappingByName;

	TArray<bool> HierarchyCurveCopied;

	TSharedPtr<FAnimNextControlRigPoseAdapter> PoseAdapter;

	// A hash to encode the pointer to anim instance
	TOptional<int32> RefPoseSetterHash;
};

} // end namespace UE::UAF
