// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SubclassOf.h"
#include "UAF/UAFAssetData.h"
#include "UObject/UObjectGlobals.h"

namespace UE::UAF::ControlRig
{

struct FAnimNextControlRigVariableProvider;

class UAFCONTROLRIG_API FAnimNextControlRigModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	// Register asset class -> graph factory mappings
	void RegisterGraphFactories() const;

	// Register asset class -> system factory mappings
	void RegisterSystemFactories() const;

	void RegisterAssetData();
	void UnregisterAssetData() const;

#if WITH_EDITOR
	using FReplacementObjectMap = FCoreUObjectDelegates::FReplacementObjectMap;
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnObjectsReinstanced, const FReplacementObjectMap&);
	static FOnObjectsReinstanced OnObjectsReinstanced;

	FDelegateHandle OnObjectsReinstancedHandle;

	TSharedPtr<FAnimNextControlRigVariableProvider> VariableProvider;
#endif

#if WITH_EDITORONLY_DATA
	FTopLevelAssetPath ControlRigBlueprintClassPath;
#endif
	FTopLevelAssetPath ControlRigBlueprintGeneratedClassPath;
};

} // end namespace
