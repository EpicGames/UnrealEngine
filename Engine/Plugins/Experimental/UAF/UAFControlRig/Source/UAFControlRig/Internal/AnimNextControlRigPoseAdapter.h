// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Tools/ControlRigPoseAdapter.h"
#include "ReferencePose.h"
#include "LODPose.h"

struct FReferenceSkeleton;
class UNodeMappingContainer;
class UControlRig;
class URigHierarchy;

namespace UE::UAF
{
struct FKeyframeState;
}

namespace UE::UAF::ControlRig
{

class UAFCONTROLRIG_API FAnimNextControlRigPoseAdapter : public FControlRigPoseAdapter
{
public:
	FAnimNextControlRigPoseAdapter() = default;

	void CopyBonesFrom(const FLODPoseStack& InPose)
	{
		InPose.CopyTransformsTo(LocalPose);
	}

	void UpdateInputOutputMappingIfRequired(UControlRig* InControlRig
		, URigHierarchy* InHierarchy
		, const UE::UAF::FReferencePose& InRefPose
		, const int32 InCurentLOD
		, const TWeakObjectPtr<UNodeMappingContainer>& InNodeMappingContainer
		, bool bInTransferPoseInGlobalSpace
		, bool bInResetInputPoseToInitial);

	static const FReferenceSkeleton* GetReferenceSkeleton(const UE::UAF::FReferencePose& InRefPose);
};

} // end namespace UE::UAF
