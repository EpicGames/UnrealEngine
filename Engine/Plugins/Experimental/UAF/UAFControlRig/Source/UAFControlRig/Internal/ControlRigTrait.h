// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitSharedData.h"
#include "TraitInterfaces/IUpdate.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IHierarchy.h"
#include "TraitInterfaces/IGarbageCollection.h"
#include "ControlRig.h"
#include "Delegates/IDelegateInstance.h"
#include "AnimNextControlRigHierarchyMappings.h"
#include "Tools/ControlRigVariableMappings.h"
#include "Tools/ControlRigIOSettings.h"
#include "ControlRigTask.h"
#if WITH_EDITOR
#include "ControlRigIOMapping.h"
#endif
// --- ---
#include "ControlRigTrait.generated.h"

class USkeletalMeshComponent;
class IControlRigEditorAssetInterface;
struct FRigVMExternalVariable;
struct FAnimNextControlRigTask;

USTRUCT()
struct FControlRigExposedProperty
{
	GENERATED_BODY()

	FControlRigExposedProperty() = default;
	FControlRigExposedProperty(const FName& InExposedPropertyName, bool bInIsvariable)
		: ExposedPropertyName(InExposedPropertyName)
		, bIsVaraible(bInIsvariable)
	{}
	
	UPROPERTY(meta = (Hidden))
	FName ExposedPropertyName = NAME_None;

	UPROPERTY(meta = (Hidden))
	bool bIsVaraible = false;
};

USTRUCT()
struct FControlRigTraitSharedDataBase : public FAnimNextTraitSharedData
{
	GENERATED_BODY()
	
	/** Input to be processed. */
	UPROPERTY()
	FAnimNextTraitHandle Input;

	// Skeleton to use as source to extract the Rig Controls. If null, the system will use the preview skeleton that was used to create the rig.
	UPROPERTY(EditAnywhere, Category = ControlRig, meta = (Hidden))
	TObjectPtr<USkeleton> ControlRigSkeleton;

	/**
	 * If this is checked the rig's pose needs to be reset to its initial
	 * prior to evaluating the rig.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Settings, meta = (Hidden))
	bool bResetInputPoseToInitial = true;

	/**
	 * If this is checked the bone pose coming from the Input will be
	 * transferred into the Control Rig.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Settings, meta = (Hidden))
	bool bTransferInputPose = true;

	/**
	 * If this is checked the curves coming from the AnimBP will be
	 * transferred into the Control Rig.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Settings, meta = (Hidden))
	bool bTransferInputCurves = true;

	/**
	* Is set, override the initial transforms with those taken from the mesh component
	*/
	UPROPERTY(EditAnywhere, Category=Settings, meta = (DisplayName = "Set Initial Transforms From Mesh", Hidden))
	bool bSetRefPoseFromSkeleton = false;

	/**
	 * Transferring the pose in global space guarantees a global pose match,
	 * while transferring in local space ensures a match of the local transforms.
	 * In general transforms only differ if the hierarchy topology differs
	 * between the Control Rig and the skeleton used in the AnimBP.
	 * Note: Turning this off can potentially improve performance.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, meta = (Hidden), Category = Settings)
	bool bTransferPoseInGlobalSpace = false;

	// The customized event queue to run
	UPROPERTY(EditAnywhere, AdvancedDisplay, meta = (Hidden), Category = Settings)
	TArray<FControlRigEventName> EventQueue;

	/**
	 * An inclusive list of bones to transfer as part
	 * of the input pose transfer phase.
	 * If this list is empty all bones will be transferred.
	 */
	//UPROPERTY(EditAnywhere, AdvancedDisplay, meta = (Hidden), Category = Settings)
	//TArray<FBoneReference> InputBonesToTransfer; //TODO zzz : Unsupported in AnimNext 

	/**
	 * An inclusive list of bones to transfer as part
	 * of the output pose transfer phase.
	 * If this list is empty all bones will be transferred.
	 */
	//UPROPERTY(EditAnywhere, AdvancedDisplay, meta = (Hidden), Category = Settings)
	//TArray<FBoneReference> OutputBonesToTransfer; //TODO zzz : Unsupported in AnimNext 

	// we only save mapping, 
	// we have to query control rig when runtime 
	// to ensure type and everything is still valid or not
	UPROPERTY(meta = (Hidden))
	TMap<FName, FName> InputMapping;

	UPROPERTY(meta = (Hidden))
	TMap<FName, FName> OutputMapping;

	UPROPERTY(meta = (Hidden))
	TArray<FName> ExposedPropertyVariableNames;

	UPROPERTY(meta = (Hidden))
	TArray<FName> ExposedPropertyControlNames;
	
	UPROPERTY(meta = (Hidden))
	TArray<ERigControlType> ExposedPropertyControlTypes;

	UPROPERTY(meta = (Hidden))
	TArray<FString> ExposedPropertyControlDefaultValues;

	// This is the array of latent input properties
	// This is computed at load time based on the selected ControlRig asset
	UPROPERTY(Transient, meta = (Hidden))
	TArray<FName> LatentProperties;

	// This is the array that maps a latent property to its memory layout in the trait instance data
	// This is computed at load time based on the selected ControlRig asset
	UPROPERTY(Transient, meta = (Hidden))
	TArray<uint32> LatentPropertyMemoryLayouts;

	// Manual handling of latent properties
	static void InitializeLatentProperties(const TArray<FControlRigVariableMappings::FCustomPropertyData>& Mappings);

	static void DestructLatentPropertiesValues(const TArray<FControlRigVariableMappings::FCustomPropertyData>& Mappings);

	static void InitializeControlLatentPinDefaultValue(ERigControlType InControlType, uint8* InTargetLatentPinMemory);
	
};

USTRUCT(meta = (Hidden, DisplayName = "Control Rig (Legacy)"))
struct FControlRigTraitSharedData : public FControlRigTraitSharedDataBase
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = ControlRig, meta = (Hidden, DisallowCreateNew))
	TSubclassOf<UControlRig> ControlRigClass;
	
	FControlRigAssetStrongReference GetControlRigAssetReference() const { return FControlRigAssetStrongReference(ControlRigClass); }
	bool HasValidControlRigReference() const { return ControlRigClass != nullptr; }
	
	// Manual handling of latent properties
	static void ConstructLatentProperties(const UE::UAF::FTraitBinding& Binding);

	static void DestructLatentProperties(const UE::UAF::FTraitBinding& Binding);

	
#if WITH_EDITOR
	UAFCONTROLRIG_API USkeleton* GetPreviewSkeleton() const;
	TScriptInterface<const IControlRigEditorAssetInterface> GetControlRigEditorAsset() const;
#endif // WITH_EDITOR
};

USTRUCT(meta = (DisplayName = "Control Rig"))
struct FControlRigTraitSharedData_v2 : public FControlRigTraitSharedDataBase
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = ControlRig, meta = (Hidden, DisallowCreateNew))
	FControlRigAssetStrongReference ControlRigAssetReference;
	
	FControlRigAssetStrongReference GetControlRigAssetReference() const { return ControlRigAssetReference; }
	bool HasValidControlRigReference() const { return ControlRigAssetReference.IsValid(); }
	
	// Manual handling of latent properties
	static void ConstructLatentProperties(const UE::UAF::FTraitBinding& Binding);

	static void DestructLatentProperties(const UE::UAF::FTraitBinding& Binding);

	
#if WITH_EDITOR
	UAFCONTROLRIG_API USkeleton* GetPreviewSkeleton() const;
	TScriptInterface<const IControlRigEditorAssetInterface> GetControlRigEditorAsset() const;
#endif // WITH_EDITOR
};

namespace UE::UAF
{

template<typename TSharedData>
struct TControlRigTraitBase : FBaseTrait, IEvaluate, IUpdate, IHierarchy, IGarbageCollection
{
	typedef FBaseTrait Super;

	using FSharedData = TSharedData;
	
	struct FInstanceData : FTrait::FInstanceData
	{
		FTraitPtr Input;

		TSharedPtr<FAnimNextControlRigTask> EvaluationTask;

		TObjectPtr<UControlRig> ControlRig;

		FDelegateHandle OnObjectsReinstancedHandle;
		FDelegateHandle OnInitializedHandle;

		FControlRigVariableMappings::FCustomPropertyMappings PropertyMappings;

		void Construct(const FExecutionContext& Context, const FTraitBinding& Binding);
		void Destruct(const FExecutionContext& Context, const FTraitBinding& Binding);

		void InitializeControlRig(const FExecutionContext& Context, const FTraitBinding& Binding);
		void DestroyControlRig(const FExecutionContext& Context, const FTraitBinding& Binding);

		static UObject* GetAnimContext(const FExecutionContext& Context);
		static const USkeletalMeshComponent* GetBindableObject(const FExecutionContext& Context);
		static FControlRigAssetStrongReference GetTargetAssetReference(const TSharedData* SharedData);

		// Returns the property, variable name and the memory of the variable (as target) and the memory of the latent property (as source)
		static int32 GetExposedVariablesData(const UE::UAF::FTraitBinding& Binding, const TSharedData* SharedData, FControlRigVariableMappings::FCustomPropertyMappings& OutMappings);
		static int32 GetExposedControlsData(const UE::UAF::FTraitBinding& Binding, const TSharedData* SharedData, FControlRigVariableMappings::FCustomPropertyMappings& OutMappings);
		static void MapUAFVariablesToControlRig(const UE::UAF::FTraitBinding& Binding, const TSharedData* SharedData, FControlRigVariableMappings::FCustomPropertyMappings& OutMappings);

		void HandleOnInitialized_AnyThread(URigVMHost*, const FName&);

		void InitializeCustomProperties(const FTraitBinding& Binding, const TSharedData* SharedData);

#if WITH_EDITOR
		void OnObjectsReinstanced(const TMap<UObject*, UObject*>& OldToNewInstanceMap);
		bool bRefreshBindableObject = false;
		bool bReinitializeControlRig = false;
		bool bRegenerateVariableMappings = false;
#endif
	};

	// FTrait impl
	virtual void SerializeTraitSharedData(FArchive& Ar, FAnimNextTraitSharedData& SharedData) const override;
	virtual uint32 GetNumLatentTraitProperties() const override;
	virtual UE::UAF::FTraitLatentPropertyMemoryLayout GetLatentPropertyMemoryLayout(const FAnimNextTraitSharedData& SharedData, FName PropertyName, uint32 PropertyIndex) const override;

	// IUpdate impl
	virtual void OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;
	virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

	// IEvaluate impl
	virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;

	// IHierarchy impl
	virtual uint32 GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const override;
	virtual void GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const override;

	// IGarbageCollection impl
	virtual void AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const override;

protected:
	static UControlRig* GetControlRig(FInstanceData* InstanceData);

	static bool CreateControlRig(UObject* InAnimContext, UObject* InBindableObject, FControlRigAssetStrongReference InControlRigAssetReference, FInstanceData* InstanceData);
	static bool SetBindableObject(UControlRig* ControlRig, UObject* InAnimContext, UObject* InBindableObject);

	static FControlRigAssetStrongReference GetTargetAssetReference(const FSharedData* SharedData, const TTraitBinding<IUpdate>& Binding);

#if WITH_EDITOR
	virtual void GetProgrammaticPins(FAnimNextTraitSharedData* InSharedData,
		URigVMController* InController,
		int32 InParentPinIndex,
		const URigVMPin* TraitPin,
		const FString& InDefaultValue,
		struct FRigVMPinInfoArray& OutPinArray) const override;

	virtual int32 GetLatentPropertyIndex(const FAnimNextTraitSharedData& InSharedData, FName PropertyName) const override;

	virtual uint32 GetLatentPropertyHandles(
		const FAnimNextTraitSharedData* InSharedData,
		TArray<FLatentPropertyMetadata>& OutLatentPropertyHandles,
		bool bFilterEditorOnly,
		const TFunction<uint16(FName PropertyName)>& GetTraitLatentPropertyIndex) const override;

	FControlRigIOMapping::FRigControlsData RigControlsData;
#endif

	static bool GetVariableSizeAndAlignment(const FRigVMExternalVariable& Variable, uint32& PropertySize, uint32& PropertyAlignment);
	static bool GetControlSizeAndAlignment(ERigControlType ControlType, uint32& PropertySize, uint32& PropertyAlignment);
};
	
struct FControlRigTrait : TControlRigTraitBase<FControlRigTraitSharedData>
{
	using Super = TControlRigTraitBase<FControlRigTraitSharedData>;
	
	ANIM_NEXT_IMPL_DECLARE_ANIM_TRAIT_BASIC(FControlRigTrait, FBaseTrait)
	ANIM_NEXT_IMPL_DECLARE_ANIM_TRAIT_INSTANCING_SUPPORT()
	ANIM_NEXT_IMPL_DECLARE_ANIM_TRAIT_INTERFACE_SUPPORT()
	ANIM_NEXT_IMPL_DECLARE_ANIM_TRAIT_EVENT_SUPPORT()
};
	
struct FControlRigTrait_v2 : TControlRigTraitBase<FControlRigTraitSharedData_v2>
{
	using Super = TControlRigTraitBase<FControlRigTraitSharedData_v2>;

	ANIM_NEXT_IMPL_DECLARE_ANIM_TRAIT_BASIC(FControlRigTrait_v2, FBaseTrait)
	ANIM_NEXT_IMPL_DECLARE_ANIM_TRAIT_INSTANCING_SUPPORT()
	ANIM_NEXT_IMPL_DECLARE_ANIM_TRAIT_INTERFACE_SUPPORT()
	ANIM_NEXT_IMPL_DECLARE_ANIM_TRAIT_EVENT_SUPPORT()
};

} // namespace UE::UAF
