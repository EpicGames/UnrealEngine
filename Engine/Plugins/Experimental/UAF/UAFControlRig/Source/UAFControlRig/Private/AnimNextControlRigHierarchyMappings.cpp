// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextControlRigHierarchyMappings.h"

#include "ControlRig.h"
#include "BoneContainer.h"
#include "Rigs/RigHierarchy.h"
#include "Animation/NodeMappingContainer.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimCurveTypes.h"
#include "HAL/IConsoleManager.h"
#include "Tools/ControlRigIOSettings.h"
#include "Tools/ControlRigPoseAdapter.h"
#include "Components/SkeletalMeshComponent.h"
#include "EvaluationVM/KeyframeState.h"
#include "AnimNextControlRigPoseAdapter.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/Skeleton.h"

namespace UE::UAF::ControlRig
{

void FAnimNextControlRigHierarchyMappings::InitializeInstance()
{
	PoseAdapter = MakeShareable(new FAnimNextControlRigPoseAdapter());
}

void FAnimNextControlRigHierarchyMappings::LinkToHierarchy(URigHierarchy* InHierarchy)
{
	if (InHierarchy)
	{
		InHierarchy->LinkPoseAdapter(PoseAdapter);
	}
}

void FAnimNextControlRigHierarchyMappings::UpdateControlRigRefPoseIfNeeded(UControlRig* ControlRig
	, const UObject* InstanceObject
	, const USkeletalMeshComponent* SkeletalMeshComponent
	, const UE::UAF::FReferencePose& InRefPose
	, bool bInSetRefPoseFromSkeleton
	, bool bIncludePoseInHash)
{
	check(ControlRig);

	if (!bInSetRefPoseFromSkeleton)
	{
		return;
	}

	int32 ExpectedHash = 0;
	ExpectedHash = HashCombine(ExpectedHash, (int32)reinterpret_cast<uintptr_t>(InstanceObject));
	ExpectedHash = HashCombine(ExpectedHash, (int32)reinterpret_cast<uintptr_t>(SkeletalMeshComponent));

	if (SkeletalMeshComponent)
	{
		ExpectedHash = HashCombine(ExpectedHash, (int32)reinterpret_cast<uintptr_t>(SkeletalMeshComponent->GetSkeletalMeshAsset()));
	}

	if (bIncludePoseInHash)
	{
		const int32 NumRefTransforms = InRefPose.ReferenceLocalTransforms.Num();
		for (int32 Index = 0; Index < NumRefTransforms; Index++)
		{
			const FTransform& Transform = InRefPose.ReferenceLocalTransforms[Index];
			const FQuat Rotation = Transform.GetRotation();

			ExpectedHash = HashCombine(ExpectedHash, GetTypeHash(Transform.GetTranslation()));
			ExpectedHash = HashCombine(ExpectedHash, GetTypeHash(FVector4(Rotation.X, Rotation.Y, Rotation.Z, Rotation.W)));
			ExpectedHash = HashCombine(ExpectedHash, GetTypeHash(Transform.GetScale3D()));
		}
	}

	if (RefPoseSetterHash.IsSet() && (ExpectedHash == RefPoseSetterHash.GetValue()))
	{
		return;
	}

	const FReferenceSkeleton* RefSkeleton = FAnimNextControlRigPoseAdapter::GetReferenceSkeleton(InRefPose);
	check(RefSkeleton != nullptr);
	ControlRig->SetBoneInitialTransformsFromRefSkeleton(*RefSkeleton);

	RefPoseSetterHash = ExpectedHash;
}

void FAnimNextControlRigHierarchyMappings::UpdateInputOutputMappingIfRequired(UControlRig* InControlRig
	, URigHierarchy* InHierarchy
	, const UE::UAF::FReferencePose& InRefPose
	, int32 InCurrentLOD
	, const TArray<FBoneReference>& InInputBonesToTransfer
	, const TArray<FBoneReference>& InOutputBonesToTransfer
	, const TWeakObjectPtr<UNodeMappingContainer>& InNodeMappingContainer
	, bool bInTransferPoseInGlobalSpace
	, bool bInResetInputPoseToInitial)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (InHierarchy == nullptr)
	{
		return;
	}

	InHierarchy->UnlinkPoseAdapter();

	if (!PoseAdapter.IsValid())
	{
		return;
	}

	PoseAdapter->UpdateInputOutputMappingIfRequired(InControlRig
		, InHierarchy
		, InRefPose
		, InCurrentLOD
		, InNodeMappingContainer
		, bInTransferPoseInGlobalSpace
		, bInResetInputPoseToInitial);
}

void FAnimNextControlRigHierarchyMappings::UpdateInput(UControlRig* ControlRig
	, UE::UAF::FKeyframeState& InOutput
	, const FControlRigIOSettings& InInputSettings
	, const FControlRigIOSettings& InOutputSettings
	, const TWeakObjectPtr<UNodeMappingContainer>& InNodeMappingContainer
	, bool bInExecute
	, bool bInTransferInputPose
	, bool bInResetInputPoseToInitial
	, bool bInTransferPoseInGlobalSpace
	, bool bInTransferInputCurves)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
	if (Hierarchy == nullptr)
	{
		return;
	}

#if WITH_EDITOR
	// if we are recording any change - let's clear the undo stack
	if (bInExecute && Hierarchy->IsTracingChanges())
	{
		Hierarchy->ResetTransformStack();
	}
#endif

	if (InOutputSettings.bUpdatePose && PoseAdapter)
	{
		if (InInputSettings.bUpdatePose)
		{
			// reset transforms here to prevent additive transforms from accumulating to INF
			// we only update transforms from the mesh pose for bones in the current LOD, 
			// so the reset here ensures excluded bones are also reset
			if (!PoseAdapter->GetBonesToResetToInitial().IsEmpty())
			{
				FRigHierarchyValidityBracket ValidityBracket(Hierarchy);
				{
#if WITH_EDITOR
					// make sure transient controls don't get reset
					UControlRig::FTransientControlPoseScope PoseScope(ControlRig);
#endif
					for (const int32& BoneElementIndex : PoseAdapter->GetBonesToResetToInitial())
					{
						if (FRigTransformElement* BoneElement = Hierarchy->Get<FRigBoneElement>(BoneElementIndex))
						{
							const FTransform InitialLocalTransform = Hierarchy->GetTransform(BoneElement, ERigTransformType::InitialLocal);
							BoneElement->GetTransform().Set(ERigTransformType::CurrentLocal, InitialLocalTransform);
							BoneElement->GetDirtyState().MarkClean(ERigTransformType::CurrentLocal);
							BoneElement->GetDirtyState().MarkDirty(ERigTransformType::CurrentGlobal);
						}
					}
				}
			}

			if (bInTransferInputPose && InOutput.Pose.GetNumBones() == PoseAdapter->GetLocalPose().Num())
			{
				PoseAdapter->MarkDependentsDirty();
				PoseAdapter->CopyBonesFrom(InOutput.Pose);
				PoseAdapter->UpdateDirtyStates(true);
			}
		}

		if (InInputSettings.bUpdateCurves || InOutputSettings.bUpdateCurves)
		{
			Hierarchy->UnsetCurveValues();

			const TArray<FRigBaseElement*> HierarchyCurves = Hierarchy->GetCurvesFast();
			PoseAdapter->SetHierarchyCurvesLookup(HierarchyCurves);
			PoseAdapter->SetPoseCurveToHierarchyCurve(HierarchyCurves, InOutput.Curves);
		}
	}

#if WITH_EDITOR
	if (bInExecute && Hierarchy->IsTracingChanges())
	{
		Hierarchy->StorePoseForTrace(TEXT("FAnimNode_ControlRigBase::UpdateInput"));
	}
#endif
}

void FAnimNextControlRigHierarchyMappings::UpdateOutput(UControlRig* ControlRig
	, UE::UAF::FKeyframeState& InOutput
	, const FControlRigIOSettings& InOutputSettings
	, const TWeakObjectPtr<UNodeMappingContainer>& InNodeMappingContainer
	, bool bInExecute
	, bool bInTransferPoseInGlobalSpace)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
	if (Hierarchy == nullptr)
	{
		return;
	}

	if (InOutputSettings.bUpdatePose && PoseAdapter)
	{
		// make sure the local respectively global transforms are all ready
		if (PoseAdapter->GetTransferInLocalSpace())
		{
			PoseAdapter->ConvertToLocalPose();
		}
		else
		{
			PoseAdapter->ConvertToGlobalPose();
		}

		// if we transfered in global - let's convert back to local
		if (!PoseAdapter->GetTransferInLocalSpace())
		{
			if (PoseAdapter->GetGlobalPose().Num() == InOutput.Pose.GetNumBones())
			{
				PoseAdapter->ConvertToLocalPose();
			}
		}

		InOutput.Pose.CopyTransformsFrom(PoseAdapter->GetLocalPose());
	}

	if (InOutputSettings.bUpdateCurves)
	{
		const TArray<int32>& ChangedCurveIndices = Hierarchy->GetChangedCurveIndices();
		if (ChangedCurveIndices.Num() > 0)
		{
			const TArray<FRigBaseElement*> HierarchyCurves = Hierarchy->GetCurvesFast();

			int32 CurveIndex = 0;
			int32 CurvesCopied = 0;
			InOutput.Curves.ForEachElement([this, &CurveIndex, &CurvesCopied, &HierarchyCurves](const UE::Anim::FCurveElement& InCurveElement)
				{
					// the index stored here is the sub index of the curve (the index of the curve within the list of curves)
					const int32& HierarchyIndex = PoseAdapter->GetPoseCurveToHierarchyCurve()[CurveIndex];
					if (HierarchyIndex != INDEX_NONE)
					{
						const FRigCurveElement* HierarchyCurve = CastChecked<FRigCurveElement>(HierarchyCurves[HierarchyIndex]);
						if (HierarchyCurve->IsValueSet())
						{
							const_cast<UE::Anim::FCurveElement*>(&InCurveElement)->Value = HierarchyCurve->Get();
						}
						CurvesCopied++;
					}
					CurveIndex++;
				});

			if (CurvesCopied < HierarchyCurves.Num())
			{
				HierarchyCurveCopied.SetNumUninitialized(Hierarchy->Num());
				FMemory::Memzero(HierarchyCurveCopied.GetData(), HierarchyCurveCopied.GetAllocatedSize());

				FBlendedCurve ControlRigCurves;
				ControlRigCurves.Reserve(ChangedCurveIndices.Num());
				for (const int32& ChangedCurveIndex : ChangedCurveIndices)
				{
					if (!HierarchyCurveCopied[ChangedCurveIndex])
					{
						if (const FRigCurveElement* HierarchyCurve = Hierarchy->Get<FRigCurveElement>(ChangedCurveIndex))
						{
							if (HierarchyCurve->IsValueSet())
							{
								ControlRigCurves.Add(HierarchyCurve->GetFName(), HierarchyCurve->Get());
							}
						}
						HierarchyCurveCopied[ChangedCurveIndex] = true;
					}
				}

				InOutput.Curves.Combine(ControlRigCurves);
			}
		}
	}

#if WITH_EDITOR
	if (bInExecute && Hierarchy->IsTracingChanges())
	{
		Hierarchy->StorePoseForTrace(TEXT("FAnimNode_ControlRigBase::UpdateOutput"));
		Hierarchy->DumpTransformStackToFile();
	}
#endif
}

bool FAnimNextControlRigHierarchyMappings::CheckPoseAdapter() const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (!PoseAdapter.IsValid())
	{
		return false;
	}
	return true;
}

bool FAnimNextControlRigHierarchyMappings::IsUpdateToDate(const URigHierarchy* InHierarchy) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (!PoseAdapter->IsUpdateToDate(InHierarchy))
	{
		return false;
	}
	return true;
}

void FAnimNextControlRigHierarchyMappings::PerformUpdateToDate(UControlRig* ControlRig
	, URigHierarchy* InHierarchy
	, const UE::UAF::FReferencePose& InRefPose
	, int32 InCurrentLOD
	, const TWeakObjectPtr<UNodeMappingContainer>& InNodeMappingContainer
	, bool bInTransferPoseInGlobalSpace
	, bool bInResetInputPoseToInitial)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (!PoseAdapter->IsUpdateToDate(InHierarchy))
	{
		InHierarchy->UnlinkPoseAdapter();
		PoseAdapter->UpdateInputOutputMappingIfRequired(ControlRig, InHierarchy, InRefPose, InCurrentLOD, InNodeMappingContainer, bInTransferPoseInGlobalSpace, bInResetInputPoseToInitial);
		InHierarchy->LinkPoseAdapter(PoseAdapter);
	}
}

} // namespace UE::UAF::ControlRig
