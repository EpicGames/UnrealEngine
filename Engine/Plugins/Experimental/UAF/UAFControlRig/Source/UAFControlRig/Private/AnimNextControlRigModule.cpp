// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextControlRigModule.h"

#include "ControlRig.h"

#include "ControlRigTrait.h"
#include "Factory/AnimGraphFactory.h"
#include "Factory/AnimNextFactoryParams.h"
#include "Factory/SystemFactory.h"
#include "Injection/InjectionSiteTrait.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#if WITH_EDITORONLY_DATA
#include "ControlRigBlueprintLegacy.h"
#endif

#if WITH_EDITOR
#include "AnimNextControlRigVariableProvider.h"
#endif
#include "Native/UAFSimplePrePhysicsGraphComponent.h"

#include "ControlRigBlueprintGeneratedClass.h"
#include "UAFControlRigAssetData.h"
#include "UAF/UAFAssetFactory.h"

struct FControlRigTraitSharedData_v2;

namespace UE::UAF::ControlRig
{

#if WITH_EDITOR
FAnimNextControlRigModule::FOnObjectsReinstanced FAnimNextControlRigModule::OnObjectsReinstanced;
#endif

void FAnimNextControlRigModule::StartupModule()
{
#if WITH_EDITOR
	// Register thread safe delegates, which allows objects to safely register to these delegates which may be called from another thread.
	OnObjectsReinstancedHandle = FCoreUObjectDelegates::OnObjectsReinstanced.AddLambda([this](const FReplacementObjectMap& ObjectMap)
		{
			if (OnObjectsReinstanced.IsBound())
			{
				OnObjectsReinstanced.Broadcast(ObjectMap);
			}
		});

	VariableProvider = MakeShared<FAnimNextControlRigVariableProvider>();
#endif
	
	FRigVMRegistry& RigVMRegistry = FRigVMRegistry::Get();
	
	static UScriptStruct* const AllowedStructTypes[] =
	{
		FControlRigAssetStrongReference::StaticStruct()
	};
	
	static TPair<UClass*, FRigVMRegistry::ERegisterObjectOperation> const AllowedObjectTypes[] =
	{
	{ UControlRigRuntimeAsset::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
	};

	RigVMRegistry.RegisterStructTypes(AllowedStructTypes);
	RigVMRegistry.RegisterObjectTypes(AllowedObjectTypes);

	RegisterGraphFactories();
	RegisterSystemFactories();
	RegisterAssetData();
}

void FAnimNextControlRigModule::ShutdownModule()
{
	UnregisterAssetData();
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReinstanced.Remove(OnObjectsReinstancedHandle);

	VariableProvider.Reset();
#endif
}

void FAnimNextControlRigModule::RegisterGraphFactories() const
{
	FAnimNextFactoryParams Params;
	Params.AddTraitStruct<FControlRigTraitSharedData_v2>(ETraitVariableMapping::All, 0);

	FAnimGraphFactory::RegisterAsset<FUAFGraphFactoryAsset_ControlRig>(
		MoveTemp(Params),
		[](const FUAFGraphFactoryAsset_ControlRig& AssetData, FAnimNextFactoryParams& InOutParams)
			{
				ensure(
					InOutParams.AccessTraitStruct<FControlRigTraitSharedData_v2>(0, [&AssetData, &InOutParams](FControlRigTraitSharedData_v2& InStruct)
						{
						InStruct.ControlRigAssetReference = AssetData.ControlRigAssetReference;
						}));

				// Add control rig variables to public api for the graph
				if (AssetData.ControlRigAssetReference.IsValid())
				{
					InOutParams.AddPublicVariablesRigVMAsset(AssetData.ControlRigAssetReference.Get());
				}
			});
}

void FAnimNextControlRigModule::RegisterSystemFactories() const
{
	{
		FUAFSystemFactoryParams Params;
		Params.AddComponent<FUAFSimplePrePhysicsGraphComponent>();
		Params.AddPublicVariablesStruct<FControlRigTraitSharedData_v2>(); // Q: do I need this? or is it handled by the graph factory?
		Params.AddPublicVariablesStruct<FInjectionSiteTraitData>();

		// Register for cooked asset
		FSystemFactory::RegisterAsset<FUAFGraphFactoryAsset_ControlRig>(
			MoveTemp(Params),
			[](const FUAFGraphFactoryAsset_ControlRig& AssetData, FUAFSystemFactoryParams& Params)
				{
					ensure(Params.AccessVariablesStruct<FInjectionSiteTraitData>([&AssetData, &Params](FInjectionSiteTraitData& InStruct)
						{
							InStruct.Graph.Asset = AssetData.ControlRigAssetReference.Get();
						}));

					ensure(Params.AccessVariablesStruct<FControlRigTraitSharedData_v2>([&AssetData, &Params](FControlRigTraitSharedData_v2& InStruct)
						{
							InStruct.ControlRigAssetReference = AssetData.ControlRigAssetReference;
						}));

					if (AssetData.ControlRigAssetReference.IsValid())
					{
						Params.AddPublicVariablesRigVMAsset(AssetData.ControlRigAssetReference.Get());
					}
				});
	}
}

void FAnimNextControlRigModule::RegisterAssetData()
{
#if WITH_EDITORONLY_DATA
	ControlRigBlueprintClassPath = UE::UAF::FAssetDataFactory::RegisterAsset<FUAFGraphFactoryAsset_ControlRig, UControlRigBlueprint>(
		[](const UControlRigBlueprint* Blueprint)
			{
				return FUAFGraphFactoryAsset_ControlRig(Blueprint->GetControlRigAssetReference().GetBlueprintClass());
			}) ;
#endif // WITH_EDITORONLY_DATA

	ControlRigBlueprintGeneratedClassPath = UE::UAF::FAssetDataFactory::RegisterAsset<FUAFGraphFactoryAsset_ControlRig, UControlRigBlueprintGeneratedClass>(
		[](const UControlRigBlueprintGeneratedClass* BlueprintClass)
			{
				return FUAFGraphFactoryAsset_ControlRig(const_cast<UControlRigBlueprintGeneratedClass*>(BlueprintClass));
			}) ;
}

void FAnimNextControlRigModule::UnregisterAssetData() const
{
#if WITH_EDITORONLY_DATA
	FAssetDataFactory::UnregisterAsset(ControlRigBlueprintClassPath);
#endif // WITH_EDITORONLY_DATA
	FAssetDataFactory::UnregisterAsset(ControlRigBlueprintGeneratedClassPath);
}
} // end namespace

IMPLEMENT_MODULE(UE::UAF::ControlRig::FAnimNextControlRigModule, UAFControlRig)
