// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextControlRigPoseAdapter.h"
#include "Rigs/RigHierarchy.h"
#include "ControlRig.h"
#include "Animation/NodeMappingContainer.h"
#include "Engine/SkeletalMesh.h"

namespace UE::UAF::ControlRig
{

void FAnimNextControlRigPoseAdapter::UpdateInputOutputMappingIfRequired(UControlRig* InControlRig
	, URigHierarchy* InHierarchy
	, const UE::UAF::FReferencePose& InRefPose
	, const int32 InCurentLOD
	, const TWeakObjectPtr<UNodeMappingContainer>& InNodeMappingContainer
	, bool bInTransferPoseInGlobalSpace
	, bool bInResetInputPoseToInitial)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (InHierarchy == nullptr)
	{
		return;
	}

	ParentPoseIndices.Reset();
	RequiresHierarchyForSpaceConversion.Reset();
	ElementIndexToPoseIndex.Reset();
	PoseIndexToElementIndex.Reset();
	GlobalPose.Reset();
	LocalPose.Reset();
	HierarchyCurveLookup.Reset();

	const int32 NumBonesInPose = InRefPose.GetNumBonesForLOD(InCurentLOD);

	ParentPoseIndices.Reserve(NumBonesInPose);
	RequiresHierarchyForSpaceConversion.Reserve(NumBonesInPose);
	GlobalPose.AddDefaulted(NumBonesInPose);
	LocalPose.AddDefaulted(NumBonesInPose);

	bTransferInLocalSpace = !(bInTransferPoseInGlobalSpace || InNodeMappingContainer.IsValid());

	ParentPoseIndices.SetNumUninitialized(NumBonesInPose);
	for (int32 Index = 0; Index < NumBonesInPose; Index++)
	{
		ParentPoseIndices[Index] = FCompactPoseBoneIndex(InRefPose.GetLODParentBoneIndex(InCurentLOD, Index));
	}
	RequiresHierarchyForSpaceConversion.AddDefaulted(NumBonesInPose);

	UpdateDirtyStates();

	TArray<int32> MappedBoneElementIndices;
	if (NumBonesInPose > 0)
	{
		ElementIndexToPoseIndex.Reserve(NumBonesInPose);
		PoseIndexToElementIndex.Reserve(NumBonesInPose);

		const FReferenceSkeleton* RefSkeleton = GetReferenceSkeleton(InRefPose);
		if (const USkeleton* Skeleton = InRefPose.GetSkeletonAsset())
		{
			RefSkeleton = &Skeleton->GetReferenceSkeleton();
		}

		// @todo: thread-safe? probably not in editor, but it may not be a big issue in editor
		if (InNodeMappingContainer.IsValid())
		{
			// get target to source mapping table - this is reversed mapping table
			TMap<FName, FName> TargetToSourceMappingTable;
			InNodeMappingContainer->GetTargetToSourceMappingTable(TargetToSourceMappingTable);

			// now fill up node name
			for (uint16 Index = 0; Index < NumBonesInPose; ++Index)
			{
				// get bone name, and find reverse mapping
				const FSkeletonPoseBoneIndex BoneIndex(InRefPose.GetSkeletonBoneIndexFromLODBoneIndex(Index));
				if (BoneIndex.IsValid())
				{
					const FName TargetNodeName = RefSkeleton->GetBoneName(BoneIndex.GetInt());
					const FName* SourceName = TargetToSourceMappingTable.Find(TargetNodeName);
					if (SourceName)
					{
						const int32 ElementIndex = InHierarchy->GetIndex({ *SourceName, ERigElementType::Bone });
						if (ElementIndex != INDEX_NONE)
						{
							MappedBoneElementIndices.Add(ElementIndex);
							ElementIndexToPoseIndex.Add(static_cast<uint16>(ElementIndex), Index);
							PoseIndexToElementIndex.Add(ElementIndex);
							LocalPose[Index] = InHierarchy->GetLocalTransform(ElementIndex);
							GlobalPose[Index] = InHierarchy->GetGlobalTransform(ElementIndex);
							continue;
						}
					}
				}
				PoseIndexToElementIndex.Add(INDEX_NONE);
			}
		}
		else
		{
			TArray<FName> NodeNames;
			TArray<FNodeItem> NodeItems;
			InControlRig->GetMappableNodeData(NodeNames, NodeItems);

			// even if not mapped, we map only node that exists in the control rig
			for (uint16 Index = 0; Index < NumBonesInPose; ++Index)
			{
				const FSkeletonPoseBoneIndex BoneIndex(InRefPose.GetSkeletonBoneIndexFromLODBoneIndex(Index));
				if (BoneIndex.IsValid())
				{
					const FName BoneName = RefSkeleton->GetBoneName(BoneIndex.GetInt());
					if (NodeNames.Contains(BoneName))
					{
						const int32 ElementIndex = InHierarchy->GetIndex({ BoneName, ERigElementType::Bone });
						if (ElementIndex != INDEX_NONE)
						{
							MappedBoneElementIndices.Add(ElementIndex);
							ElementIndexToPoseIndex.Add(static_cast<uint16>(ElementIndex), Index);
							PoseIndexToElementIndex.Add(ElementIndex);
							LocalPose[Index] = InHierarchy->GetLocalTransform(ElementIndex);
							GlobalPose[Index] = InHierarchy->GetGlobalTransform(ElementIndex);
							continue;
						}
					}
				}
				PoseIndexToElementIndex.Add(INDEX_NONE);
			}
		}

		// once we know all of the bones we are going to transfer - we can check if any of these bones has a different parenting
		// relationship in the skeleton used in the anim graph vs the hierarchy in the rig. in that case we have to transfer in global
		if (bTransferInLocalSpace)
		{
			for (const int32& BoneElementIndex : MappedBoneElementIndices)
			{
				const int32 HierarchyParentIndex = InHierarchy->GetFirstParent(BoneElementIndex);
				const int16 PoseIndex = ElementIndexToPoseIndex.FindChecked(BoneElementIndex);
				const FBoneIndexType ParentIndex = ParentPoseIndices[PoseIndex].GetBoneIndex();

				FName HierarchyParentName(NAME_None);
				FName PoseParentName(NAME_None);

				if (HierarchyParentIndex != INDEX_NONE)
				{
					HierarchyParentName = InHierarchy->Get(HierarchyParentIndex)->GetFName();
				}
				if (ParentIndex != INVALID_BONE_INDEX)
				{
					FBoneIndexType SkeletonIndex = InRefPose.GetSkeletonBoneIndexFromLODBoneIndex(ParentIndex);
					if (SkeletonIndex != INVALID_BONE_INDEX && RefSkeleton->IsValidIndex((int32)SkeletonIndex))
					{
						PoseParentName = RefSkeleton->GetBoneName((int32)SkeletonIndex);
					}
				}

				if (HierarchyParentName.IsEqual(PoseParentName, ENameCase::CaseSensitive))
				{
					continue;
				}

				RequiresHierarchyForSpaceConversion[PoseIndex] = true;
				check(PoseIndexToElementIndex[PoseIndex] != INDEX_NONE);
				bTransferInLocalSpace = false;
			}
		}

		// only reset the full pose if we are not mapping all bones
		const TArray<FRigBaseElement*>& HierarchyBones = InHierarchy->GetBonesFast();
		const bool bMapsAllBones = MappedBoneElementIndices.Num() == HierarchyBones.Num();
		BonesToResetToInitial.Reset();
		bRequiresResetPoseToInitial = bInResetInputPoseToInitial && !bMapsAllBones;

		if (bRequiresResetPoseToInitial)
		{
			BonesToResetToInitial.Reserve(HierarchyBones.Num() - MappedBoneElementIndices.Num());

			// bone is mapped stores sub indices (bone index within the list of bones)
			TArray<bool> BoneIsMapped;
			BoneIsMapped.AddZeroed(HierarchyBones.Num());
			for (const int32& MappedTransformIndex : MappedBoneElementIndices)
			{
				const FRigBaseElement* MappedElement = InHierarchy->Get(MappedTransformIndex);
				check(MappedElement);
				BoneIsMapped[MappedElement->GetSubIndex()] = true;
			}

			// when we want to know which bones to reset we want to convert back to a global index
			for (int32 UnmappedBoneIndex = 0; UnmappedBoneIndex < BoneIsMapped.Num(); UnmappedBoneIndex++)
			{
				if (!BoneIsMapped[UnmappedBoneIndex])
				{
					BonesToResetToInitial.Add(HierarchyBones[UnmappedBoneIndex]->GetIndex());
				}
			}
		}
	}
}

const FReferenceSkeleton* FAnimNextControlRigPoseAdapter::GetReferenceSkeleton(const UE::UAF::FReferencePose& InRefPose)
{
	const FReferenceSkeleton* ReferenceSkeleton = nullptr;

	if (const USkeletalMesh* SkeletalMesh = InRefPose.GetSkeletalMeshAsset())
	{
		ReferenceSkeleton = &SkeletalMesh->GetRefSkeleton();
	}
	else if (const USkeleton* Skeleton = InRefPose.GetSkeletonAsset())
	{
		ReferenceSkeleton = &Skeleton->GetReferenceSkeleton();
	}

	return ReferenceSkeleton;
}

} // namespace UE::UAF::ControlRig
