// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextControlRigVariableProvider.h"

#include "ControlRig.h"
#include "RigVMRuntimeAsset.h"

FAnimNextVariableReference UE::UAF::ControlRig::GetAnimNextVariableReferenceFromRigVMExternalVariable(const FRigVMExternalVariable& RigVMVariable,
	const IRigVMRuntimeAssetInterface* Asset)
{
	return FAnimNextVariableReference::FromName(RigVMVariable.GetName(), Cast<const UObject>(Asset));
}

#if WITH_EDITOR

#include "AnimNextExports.h"
#include "ControlRigBlueprintLegacy.h"
#include "Engine/SkeletalMesh.h"
#include "UObject/AssetRegistryTagsContext.h"

namespace UE::UAF::ControlRig
{
FAnimNextControlRigVariableProvider::FAnimNextControlRigVariableProvider()
{
	OnGetExtraObjectTagsHandle = UObject::FAssetRegistryTag::OnGetExtraObjectTagsWithContext.AddStatic(&FAnimNextControlRigVariableProvider::GetAssetRegistryTags);
}

FAnimNextControlRigVariableProvider::~FAnimNextControlRigVariableProvider()
{
	UObject::FAssetRegistryTag::OnGetExtraObjectTagsWithContext.Remove(OnGetExtraObjectTagsHandle);
}

void FAnimNextControlRigVariableProvider::GetAssetRegistryTags(FAssetRegistryTagsContext Context)
{
	// UAF variable asset pickers work via the asset registry, so we need to export variable declarations for all control rigs

	// 1. Legacy control rigs (blueprints)
	/*
	if (const UControlRigBlueprint* ControlRig = Cast<UControlRigBlueprint>(Context.GetObject()))
	{
		FAnimNextAssetRegistryExports Exports;
		
		for (const FRigVMGraphVariableDescription& VariableDesc : ControlRig->GetAssetVariables())
		{
			if(!VariableDesc.bPublic)
			{
				continue;
			}
			
			FAnimNextParamType AnimNextType = FAnimNextParamType::FromString(VariableDesc.CPPType);
			FGuid Guid = ControlRig->FindBlueprintPropertyGuidFromName(VariableDesc.Name);
			FAnimNextExport ParameterExport = FAnimNextExport::MakeExport<FAnimNextVariableDeclarationData>(VariableDesc.Name, AnimNextType, Guid, EAnimNextExportedVariableFlags::Public | EAnimNextExportedVariableFlags::Declared);
			
			Exports.Exports.Add(ParameterExport);
		}

		
		FString TagValue;
		FAnimNextAssetRegistryExports::StaticStruct()->ExportText(TagValue, &Exports, nullptr, nullptr, PPF_None, nullptr);
		Context.AddTag(UObject::FAssetRegistryTag(UE::UAF::ExportsAnimNextAssetRegistryTag, TagValue, UObject::FAssetRegistryTag::TT_Hidden));
	}
	
	else*/
	if (const IRigVMEditorAssetInterface* RigVMAsset = Cast<IRigVMEditorAssetInterface>(Context.GetObject()))
	{
		FAnimNextAssetRegistryExports Exports;

		for (const FRigVMGraphVariableDescription& VariableDesc : RigVMAsset->GetAssetVariables())
		{
			if(!VariableDesc.bPublic)
			{
				continue;
			}
			
			FAnimNextParamType AnimNextType = FAnimNextParamType::FromString(VariableDesc.CPPType);
			if (const FProperty* Property = RigVMAsset->GetRuntimeAssetInterface()->FindGeneratedPropertyByName(VariableDesc.Name))
			{
				FGuid Guid = FGuid::NewDeterministicGuid(Property->GetPathName());
				FAnimNextExport ParameterExport = FAnimNextExport::MakeExport<FAnimNextVariableDeclarationData>(VariableDesc.Name, AnimNextType, Guid, EAnimNextExportedVariableFlags::Public | EAnimNextExportedVariableFlags::Declared);
				Exports.Exports.Add(ParameterExport);
			}
		}

		
		FString TagValue;
		FAnimNextAssetRegistryExports::StaticStruct()->ExportText(TagValue, &Exports, nullptr, nullptr, PPF_None, nullptr);
		Context.AddTag(UObject::FAssetRegistryTag(UE::UAF::ExportsAnimNextAssetRegistryTag, TagValue, UObject::FAssetRegistryTag::TT_Hidden));
		
	}
}
}

#endif // #if WITH_EDITOR

