// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "Variables/AnimNextVariableReference.h"

class IRigVMRuntimeAssetInterface;
struct FRigVMExternalVariable;

namespace UE::UAF::ControlRig
{
UAFCONTROLRIG_API FAnimNextVariableReference GetAnimNextVariableReferenceFromRigVMExternalVariable(const FRigVMExternalVariable& RigVMVariable, const IRigVMRuntimeAssetInterface* Asset);

#if WITH_EDITOR
struct FAnimNextControlRigVariableProvider
{
public:
	FAnimNextControlRigVariableProvider();
	~FAnimNextControlRigVariableProvider();

	static FAnimNextVariableReference GetVariableReferenceFromControlRig(const FRigVMExternalVariable& RigVMVariable, const IRigVMRuntimeAssetInterface* Asset);
	
protected:
	static void GetAssetRegistryTags(FAssetRegistryTagsContext Context);

private:
	FDelegateHandle OnGetExtraObjectTagsHandle;
};
#endif // WITH_EDITOR
}


