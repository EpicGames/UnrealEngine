// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigTask.h"

#include "EvaluationVM/EvaluationVM.h"
#include "EvaluationVM/KeyframeState.h"
#include "TransformArrayOperations.h"
#include "BoneContainer.h"
#include "Units/Execution/RigUnit_BeginExecution.h"
#include "Math/Transform.h"
#include "GenerationTools.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/AnimNodeBase.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "ControlRigObjectBinding.h"
#include "Misc/ScopeLock.h"
#if ENABLE_ANIM_DEBUG
#include "AnimNode_ControlRigBase.h"
#endif

static TAutoConsoleVariable<int32> CVarControlRigDisableExecutionAnimNext(TEXT("ControlRig.DisableExecutionInAnimNext"), 0, TEXT("if nonzero we disable the execution of Control Rigs inside Anim Next Trrait."));

DEFINE_STAT(STAT_AnimNext_Task_ControlRig);


void FAnimNextControlRigTask::Initialize(bool bInitControlRig, const FControlRigVariableMappings::FCustomPropertyMappings* PropertyMappings)
{
	if (Params.ControlRig)
	{
		// Provide available properties to the construction event
		if (PropertyMappings)
		{
			Params.ControlRigVariableMappings.InitializeCustomProperties(Params.ControlRig, *PropertyMappings);
			Params.ControlRigVariableMappings.PropagateCustomInputProperties(Params.ControlRig);
		}

		Params.ControlRigHierarchyMappings.InitializeInstance();

		if (Params.ControlRig->GetObjectBinding())
		{
			if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(FControlRigObjectBinding::GetBindableObject(Params.ControlRig->GetObjectBinding()->GetBoundObject())))
			{
				Params.ControlRig->GetDataSourceRegistry()->RegisterDataSource(UControlRig::OwnerComponent, SkeletalMeshComponent);
			}
		}

		if (bInitControlRig)
		{
			Params.ControlRig->Initialize(true);
			Params.ControlRig->RequestInit();
		}

		Params.ControlRigHierarchyMappings.ResetRefPoseSetterHash();
		Params.ControlRigVariableMappings.ResetCurvesInputToControlCache();
		Params.ControlRigVariableMappings.CacheCurveMappings(Params.InputMapping, Params.OutputMapping, Params.ControlRig->GetHierarchy());

		if (PropertyMappings)
		{
			// Re-init Custom Properties after construction, as new controls could be created and might have to be remapped
			Params.ControlRigVariableMappings.InitializeCustomProperties(Params.ControlRig, *PropertyMappings);
			Params.ControlRigVariableMappings.PropagateCustomInputProperties(Params.ControlRig);
		}
	}
}

void FAnimNextControlRigTask::RequestInit()
{
	if (UControlRig* ControlRig = GetControlRig())
	{
		ControlRig->RequestInit();
	}
	Params.bControlRigRequiresInitialization = true;
	Params.LastBonesSerialNumberForCacheBones = 0;
	Params.ControlRigHierarchyMappings.InitializeInstance();
	Params.ControlRigHierarchyMappings.ResetRefPoseSetterHash();
	Params.bUpdateInputOutputMapping = true;
}

void FAnimNextControlRigTask::OnControlRigInitialized()
{
	Params.ControlRigHierarchyMappings.ResetRefPoseSetterHash();
}


void FAnimNextControlRigTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	SCOPE_CYCLE_COUNTER(STAT_AnimNext_Task_ControlRig);
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	using namespace UE::UAF;

	if (VM.GetActiveNamedSet())
	{
		// TODO: Implement with new attribute runtime
		return;
	}

	TUniquePtr<FKeyframeState> KeyframeOut;

	if (Params.bConsumesPreviousPose)
	{
		// Try to get a Keyframe from the stack
		if (!VM.PopValue(KEYFRAME_STACK_NAME, KeyframeOut))
		{
			// no-op, will be caught below in validity check, here to prevent nodiscard error
		}
	}
	else if (const TUniquePtr<FKeyframeState>* SourcePose = VM.PeekValue<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0))
	{
		if (SourcePose->IsValid())
		{
			KeyframeOut = MakeUnique<FKeyframeState>(VM.MakeUninitializedKeyframe(false));
			KeyframeOut->Pose.CopyFrom((*SourcePose)->Pose);
			KeyframeOut->Curves.CopyFrom((*SourcePose)->Curves);
			KeyframeOut->Attributes.CopyFrom((*SourcePose)->Attributes);
		}
	}

	if (!KeyframeOut.IsValid())
	{
		KeyframeOut = MakeUnique<FKeyframeState>(VM.MakeReferenceKeyframe(false));
	}

	ExecuteControlRig(VM, *KeyframeOut.Get());

	// Push our blended result back
	VM.PushValue(KEYFRAME_STACK_NAME, MoveTemp(KeyframeOut));

	LastLOD = VM.GetCurrentLOD();
}

void FAnimNextControlRigTask::ExecuteControlRig(UE::UAF::FEvaluationVM& VM, UE::UAF::FKeyframeState& KeyFrameState) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (Params.ControlRig)
	{
		// Before we start modifying the RigHierarchy, we need to lock the rig to avoid corrupted state
		UE::TScopeLock LockRig(Params.ControlRig->GetEvaluateMutex());

		URigHierarchy* Hierarchy = Params.ControlRig->GetHierarchy();
		if (Hierarchy == nullptr)
		{
			return;
		}

		const UE::UAF::FLODPoseStack& Pose = KeyFrameState.Pose;
		const UE::UAF::FReferencePose& RefPose = Pose.GetRefPose();

		// remap LOD pose attributes to mesh bone pose indices
		UE::Anim::FMeshAttributeContainer MeshAttributeContainer;
		UE::UAF::FGenerationTools::RemapAttributes(Pose, KeyFrameState.Attributes, MeshAttributeContainer);

		// temporarily give control rig access to the stack allocated attribute container
		// control rig may have rig units that can add/get attributes to/from this container
		UControlRig::FAnimAttributeContainerPtrScope AttributeScope(Params.ControlRig, MeshAttributeContainer);

		const int32 CurrentLOD = VM.GetCurrentLOD();
		const bool bIsLODChange = LastLOD != CurrentLOD;
		const bool bRunPrepareForExecution = bIsLODChange || Params.bControlRigRequiresInitialization || Params.bUpdateInputOutputMapping;/* || (RefPoseBonesSerialNumber != LastRefPoseBonesSerialNumber)*/; // TODO : Detect RefPose changes
		if (bRunPrepareForExecution)
		{
			if (Params.ControlRig->IsConstructionModeEnabled() ||
				(Params.ControlRig->IsConstructionRequired() && bRunPrepareForExecution))
			{
				//UpdateGetAssetUserDataDelegate(Params.ControlRig);
				Params.ControlRig->Execute(FRigUnit_PrepareForExecution::EventName);
				Params.ControlRig->Execute(FRigUnit_PostPrepareForExecution::EventName);
			}

			// UpdateInputOutputMappingIfRequired was done in CacheBones, but there is no AnimNext equivalent
			Params.ControlRigHierarchyMappings.UpdateInputOutputMappingIfRequired(Params.ControlRig
				, Params.ControlRig->GetHierarchy()
				, RefPose
				, CurrentLOD
				, TArray<FBoneReference>()
				, TArray<FBoneReference>()
				, nullptr
				, Params.bTransferPoseInGlobalSpace
				, Params.bResetInputPoseToInitial);

			Params.bUpdateInputOutputMapping = false;
			Params.bControlRigRequiresInitialization = false;
		}

		if (!Params.ControlRigHierarchyMappings.IsUpdateToDate(Hierarchy))
		{
			Params.ControlRigHierarchyMappings.PerformUpdateToDate(Params.ControlRig
				, Hierarchy
				, RefPose
				, CurrentLOD
				, nullptr
				, Params.bTransferPoseInGlobalSpace
				, Params.bResetInputPoseToInitial);
		}

		// first update input to the system
		UpdateInput(VM, KeyFrameState, KeyFrameState);

		if (bExecute)
		{
			const TGuardValue<bool> ResetCurrentTransfromsAfterConstructionGuard = Params.ControlRig->GetResetCurrentTransformsAfterConstructionGuard(true);

#if WITH_EDITOR
			if (Hierarchy->IsTracingChanges())
			{
				Hierarchy->StorePoseForTrace(TEXT("FAnimNode_ControlRigBase::BeforeEvaluate"));
			}
#endif

			// pick the event to run
			if (Params.EventQueue.IsEmpty())
			{
				if (bClearEventQueueRequired)
				{
					Params.ControlRig->SetEventQueue({ FRigUnit_BeginExecution::EventName });
					bClearEventQueueRequired = false;
				}
			}
			else
			{
				TArray<FName> EventNames;
				Algo::Transform(Params.EventQueue, EventNames, [](const FControlRigEventName& InEventName)
					{
						return InEventName.EventName;
					});
				Params.ControlRig->SetEventQueue(EventNames);
				bClearEventQueueRequired = true;
			}

			if (Params.ControlRig->IsAdditive())
			{
				Params.ControlRig->ClearPoseBeforeBackwardsSolve();
			}

			// evaluate control rig
			// UpdateGetAssetUserDataDelegate(Params.ControlRig); // Removed in AnimNext, do we need an equivalent ?
			Params.ControlRig->Evaluate_AnyThread();

#if ENABLE_ANIM_DEBUG
#if UE_ENABLE_DEBUG_DRAWING
			// When Control Rig is at editing time (in CR editor), draw instructions are consumed by ControlRigEditMode, so we need to skip drawing here.
			const bool bShowDebug = (CVarAnimNodeControlRigDebug.GetValueOnAnyThread() == 1 && Params.ControlRig->ExecutionType != ERigExecutionType::Editing);
			if (bShowDebug)
			{
				if (Params.DebugDrawInterface)
				{
					QueueControlRigDrawInstructions(Params.DebugDrawInterface, Params.ComponentTransform);
				}
			}
#endif
#endif

#if WITH_EDITOR
			if (Hierarchy->IsTracingChanges())
			{
				Hierarchy->StorePoseForTrace(TEXT("FAnimNode_ControlRigBase::AfterEvaluate"));
			}
#endif
		}

		// now update output
		UpdateOutput(VM, KeyFrameState, KeyFrameState);
		
		// remap mesh bone index attributes back to stack container (LOD/compact bone indices)
		UE::UAF::FGenerationTools::RemapAttributes(Pose, MeshAttributeContainer, KeyFrameState.Attributes);
	}
}

void FAnimNextControlRigTask::UpdateInput(UE::UAF::FEvaluationVM& VM, UE::UAF::FKeyframeState& KeyFrameState, UE::UAF::FKeyframeState& InOutput) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (!CanExecute())
	{
		return;
	}

	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	Params.ControlRigHierarchyMappings.UpdateInput(Params.ControlRig
		, InOutput
		, InputSettings
		, OutputSettings
		, nullptr
		, bExecute
		, Params.bTransferInputPose
		, Params.bResetInputPoseToInitial
		, Params.bTransferPoseInGlobalSpace
		, Params.bTransferInputCurves);

	Params.ControlRigVariableMappings.UpdateCurveInputs(Params.ControlRig, Params.InputMapping, InOutput.Curves);
}

void FAnimNextControlRigTask::UpdateOutput(UE::UAF::FEvaluationVM& VM, UE::UAF::FKeyframeState& KeyFrameState, UE::UAF::FKeyframeState& InOutput) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (!CanExecute())
	{
		return;
	}

	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	Params.ControlRigHierarchyMappings.UpdateOutput(Params.ControlRig
		, InOutput
		, OutputSettings
		, nullptr
		, bExecute
		, Params.bTransferPoseInGlobalSpace);

	Params.ControlRigVariableMappings.UpdateCurveOutputs(Params.ControlRig, Params.OutputMapping, InOutput.Curves);
}

bool FAnimNextControlRigTask::CanExecute() const
{
	if (CVarControlRigDisableExecutionAnimNext->GetInt() != 0)
	{
		return false;
	}

	if (!Params.ControlRigHierarchyMappings.CanExecute())
	{
		return false;
	}

	if (Params.ControlRig)
	{
		return Params.ControlRig->CanExecute();
	}

	return false;
}

void FAnimNextControlRigTask::QueueControlRigDrawInstructions(FRigVMDrawInterface* InDebugDrawInterface, const FTransform& InComponentTransform) const
{
	ensure(Params.ControlRig);
	ensure(InDebugDrawInterface);

	if (Params.ControlRig && InDebugDrawInterface)
	{
		for (FRigVMDrawInstruction& Instruction : Params.ControlRig->GetDrawInterface().Instructions)
		{
			if (!Instruction.IsValid())
			{
				continue;
			}

			const FTransform InstructionTransform = Instruction.Transform * InComponentTransform;
			Instruction.Transform = InstructionTransform;
			InDebugDrawInterface->DrawInstruction(Instruction);
		}
	}
}

#if UE_ENABLE_DEBUG_DRAWING
void FAnimNextControlRigTask::SetComponentTransform(const FTransform& InComponentTransform)
{
	Params.ComponentTransform = InComponentTransform;
}
#endif