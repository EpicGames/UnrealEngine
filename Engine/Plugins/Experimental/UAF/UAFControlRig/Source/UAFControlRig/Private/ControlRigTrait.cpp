// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigTrait.h"

#include "Module/AnimNextModuleInstance.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Animation/AnimSequence.h"
#include "TraitInterfaces/IUpdate.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/ITimeline.h"
#include "TraitInterfaces/IHierarchy.h"
#include "ControlRigObjectBinding.h"
#include "ControlRigTask.h"
#include "AnimNextControlRigModule.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "AnimationDataSource.h"
#include "AnimNextControlRigVariableProvider.h"
#include "RigVMRuntimeAsset.h"
#include "RigVMCore/RigVMMemoryStorageStruct.h"
#include "TraitCore/NodeInstance.h"
#if WITH_EDITOR
#include "RigVMDeveloperTypeUtils.h"
#include "RigVMModel/RigVMController.h"
#include "ControlRigBlueprintLegacy.h" // to get the preview skeleton
#endif

//----------------------
// --- FSharedData ---
//----------------------


void FControlRigTraitSharedData::ConstructLatentProperties(const UE::UAF::FTraitBinding& Binding)
{
	const FControlRigTraitSharedData* SharedData = Binding.GetSharedData<FControlRigTraitSharedData>();
	if (SharedData && SharedData->HasValidControlRigReference())
	{
		UE::UAF::FControlRigTrait::FInstanceData* InstanceData = Binding.GetInstanceData<UE::UAF::FControlRigTrait::FInstanceData>();

		UE::UAF::FControlRigTrait::FInstanceData::GetExposedVariablesData(Binding, SharedData, InstanceData->PropertyMappings);
		UE::UAF::FControlRigTrait::FInstanceData::GetExposedControlsData(Binding, SharedData, InstanceData->PropertyMappings);
		UE::UAF::FControlRigTrait::FInstanceData::MapUAFVariablesToControlRig(Binding, SharedData, InstanceData->PropertyMappings);

		const TArray<FControlRigVariableMappings::FCustomPropertyData>& Mappings = InstanceData->PropertyMappings.GetMappings();
		InitializeLatentProperties(Mappings);
	}
}

void FControlRigTraitSharedData_v2::ConstructLatentProperties(const UE::UAF::FTraitBinding& Binding)
{
	const FControlRigTraitSharedData_v2* SharedData = Binding.GetSharedData<FControlRigTraitSharedData_v2>();
	if (SharedData && SharedData->HasValidControlRigReference())
	{
		UE::UAF::FControlRigTrait_v2::FInstanceData* InstanceData = Binding.GetInstanceData<UE::UAF::FControlRigTrait_v2::FInstanceData>();

		UE::UAF::FControlRigTrait_v2::FInstanceData::GetExposedVariablesData(Binding, SharedData, InstanceData->PropertyMappings);
		UE::UAF::FControlRigTrait_v2::FInstanceData::GetExposedControlsData(Binding, SharedData, InstanceData->PropertyMappings);
		UE::UAF::FControlRigTrait_v2::FInstanceData::MapUAFVariablesToControlRig(Binding, SharedData, InstanceData->PropertyMappings);

		const TArray<FControlRigVariableMappings::FCustomPropertyData>& Mappings = InstanceData->PropertyMappings.GetMappings();
		InitializeLatentProperties(Mappings);
	}
}

void FControlRigTraitSharedDataBase::InitializeLatentProperties(const TArray<FControlRigVariableMappings::FCustomPropertyData>& Mappings)
{
	const int32 NumProperties = Mappings.Num();

	for (const FControlRigVariableMappings::FCustomPropertyData& Mapping : Mappings)
	{
		const FProperty* Property = Mapping.Property;
		if (const uint8* LatentPinMemory = Mapping.SourceMemory)
		{
			if (Property)
			{
				// Init latent memory to default value
				uint8* MutableMemory = const_cast<uint8*>(LatentPinMemory);
				Property->InitializeValue(MutableMemory);
			}
			else if (Mapping.Type == FControlRigVariableMappings::FCustomPropertyData::ECustomPropertyType::Control)
			{
				uint8* MutableMemory = const_cast<uint8*>(LatentPinMemory);
				InitializeControlLatentPinDefaultValue(Mapping.ControlType, MutableMemory);
			}
		}
	}
}

void FControlRigTraitSharedData::DestructLatentProperties(const UE::UAF::FTraitBinding& Binding)
{
	const FControlRigTraitSharedData* SharedData = Binding.GetSharedData<FControlRigTraitSharedData>();
	if (SharedData && SharedData->HasValidControlRigReference())
	{
		UE::UAF::FControlRigTrait::FInstanceData* InstanceData = Binding.GetInstanceData<UE::UAF::FControlRigTrait::FInstanceData>();

		const TArray<FControlRigVariableMappings::FCustomPropertyData>& Mappings = InstanceData->PropertyMappings.GetMappings();
		DestructLatentPropertiesValues(Mappings);
	}
}

void FControlRigTraitSharedData_v2::DestructLatentProperties(const UE::UAF::FTraitBinding& Binding)
{
	const FControlRigTraitSharedData_v2* SharedData = Binding.GetSharedData<FControlRigTraitSharedData_v2>();
	if (SharedData && SharedData->HasValidControlRigReference())
	{
		UE::UAF::FControlRigTrait_v2::FInstanceData* InstanceData = Binding.GetInstanceData<UE::UAF::FControlRigTrait_v2::FInstanceData>();

		const TArray<FControlRigVariableMappings::FCustomPropertyData>& Mappings = InstanceData->PropertyMappings.GetMappings();
		DestructLatentPropertiesValues(Mappings);
	}
}

void FControlRigTraitSharedDataBase::DestructLatentPropertiesValues(const TArray<FControlRigVariableMappings::FCustomPropertyData>& Mappings)
{
	const int32 NumProperties = Mappings.Num();

	for (const FControlRigVariableMappings::FCustomPropertyData& Mapping : Mappings)
	{
		const FProperty* Property = Mapping.Property;
		const uint8* LatentPinMemory = Mapping.SourceMemory;
		if (Property && LatentPinMemory)
		{
			uint8* MutableMemory = const_cast<uint8*>(LatentPinMemory);
			Property->DestroyValue(MutableMemory);
		}
	}
}

// Unlike mapped variables, controls does not come with an associated property, 
// so I default init the latent property memory until the graph provides a valid value
// This avoids setting control initial values with random memory
void FControlRigTraitSharedDataBase::InitializeControlLatentPinDefaultValue(ERigControlType InControlType, uint8* InTargetLatentPinMemory)
{
	check(InTargetLatentPinMemory != nullptr);

	switch (InControlType)
	{
		case ERigControlType::Bool:
		{
			bool& ValuePtr = *(bool*)InTargetLatentPinMemory;
			ValuePtr = false;
			break;
		}
		case ERigControlType::Float:
		case ERigControlType::ScaleFloat:
		{
			float& ValuePtr = *(float*)InTargetLatentPinMemory;
			ValuePtr = 0.f;
			break;
		}
		case ERigControlType::Integer:
		{
			int32& ValuePtr = *(int32*)InTargetLatentPinMemory;
			ValuePtr = 0;
			break;
		}
		case ERigControlType::Vector2D:
		{
			FVector2D& ValuePtr = *(FVector2D*)InTargetLatentPinMemory;
			ValuePtr = FVector2D::ZeroVector;
			break;
		}
		case ERigControlType::Position:
		case ERigControlType::Scale:
		{
			FVector& ValuePtr = *(FVector*)InTargetLatentPinMemory;
			ValuePtr = FVector::ZeroVector;
			break;
		}
		case ERigControlType::Rotator:
		{
			FRotator& ValuePtr = *(FRotator*)InTargetLatentPinMemory;
			ValuePtr = FRotator::ZeroRotator;
			break;
		}
		case ERigControlType::Transform:
		case ERigControlType::TransformNoScale:
		case ERigControlType::EulerTransform:
		{
			FTransform& ValuePtr = *(FTransform*)InTargetLatentPinMemory;
			ValuePtr = FTransform::Identity;
			break;
		}
		default:
		{
			checkNoEntry();
		}
	}
}

#if WITH_EDITOR
USkeleton* FControlRigTraitSharedData::GetPreviewSkeleton() const
{
	USkeleton* PreviewSkeleton = ControlRigSkeleton;

	if (PreviewSkeleton == nullptr && HasValidControlRigReference())
	{
		// If the user has not provided an explicit skeleton to use, 
		// as AnimNext does not have a preview skeleton, I get the one that was used to generate the rig
		// (note that this might to be valid for some constructions and the user might have to provide the skeleton)
		if (TScriptInterface<const IControlRigEditorAssetInterface> RigVMBlueprint = GetControlRigEditorAsset())
		{
			if (USkeletalMesh* SkeletalMesh = RigVMBlueprint->GetPreviewMesh())
			{
				PreviewSkeleton = SkeletalMesh->GetSkeleton();
			}
		}
	}

	return PreviewSkeleton;
}

USkeleton* FControlRigTraitSharedData_v2::GetPreviewSkeleton() const
{
	USkeleton* PreviewSkeleton = ControlRigSkeleton;

	if (PreviewSkeleton == nullptr && HasValidControlRigReference())
	{
		// If the user has not provided an explicit skeleton to use, 
		// as AnimNext does not have a preview skeleton, I get the one that was used to generate the rig
		// (note that this might to be valid for some constructions and the user might have to provide the skeleton)
		if (TScriptInterface<const IControlRigEditorAssetInterface> RigVMBlueprint = GetControlRigEditorAsset())
		{
			if (USkeletalMesh* SkeletalMesh = RigVMBlueprint->GetPreviewMesh())
			{
				PreviewSkeleton = SkeletalMesh->GetSkeleton();
			}
		}
	}

	return PreviewSkeleton;
}

TScriptInterface<const IControlRigEditorAssetInterface> FControlRigTraitSharedData::GetControlRigEditorAsset() const
{
	if (ControlRigClass)
	{
		return ControlRigClass->ClassGeneratedBy;
	}
	return nullptr;
}

TScriptInterface<const IControlRigEditorAssetInterface> FControlRigTraitSharedData_v2::GetControlRigEditorAsset() const
{
	if (ControlRigAssetReference.IsValid())
	{
		return ControlRigAssetReference.GetEditorAsset();
	}
	return nullptr;
}

#endif //WITH_EDITOR

namespace UE::UAF
{

AUTO_REGISTER_ANIM_TRAIT(FControlRigTrait)
AUTO_REGISTER_ANIM_TRAIT(FControlRigTrait_v2)

// Trait implementation boilerplate
#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(IEvaluate) \
	GeneratorMacro(IUpdate) \
	GeneratorMacro(IHierarchy) \
	GeneratorMacro(IGarbageCollection) \

ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT(FControlRigTrait)
ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_GET_INTERFACE(FControlRigTrait, TRAIT_INTERFACE_ENUMERATOR)
ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_GET_INTERFACES(FControlRigTrait, TRAIT_INTERFACE_ENUMERATOR)
ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_GET_REQUIRED_INTERFACES(FControlRigTrait, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR)
ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_ON_TRAIT_EVENT(FControlRigTrait, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_GET_TRAIT_EVENTS(FControlRigTrait, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	
ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT(FControlRigTrait_v2)
ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_GET_INTERFACE(FControlRigTrait_v2, TRAIT_INTERFACE_ENUMERATOR)
ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_GET_INTERFACES(FControlRigTrait_v2, TRAIT_INTERFACE_ENUMERATOR)
ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_GET_REQUIRED_INTERFACES(FControlRigTrait_v2, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR)
ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_ON_TRAIT_EVENT(FControlRigTrait_v2, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
ANIM_NEXT_IMPL_DEFINE_ANIM_TRAIT_GET_TRAIT_EVENTS(FControlRigTrait_v2, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
#undef TRAIT_INTERFACE_ENUMERATOR

//----------------------
// --- FInstanceData ---
//----------------------

template<typename TSharedData>
void TControlRigTraitBase<TSharedData>::FInstanceData::Construct(const FExecutionContext& Context, const FTraitBinding& Binding)
{
	FTrait::FInstanceData::Construct(Context, Binding);

	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	check(SharedData);

#if WITH_EDITOR
	{
		OnObjectsReinstancedHandle = UE::UAF::ControlRig::FAnimNextControlRigModule::OnObjectsReinstanced.AddRaw(this, &FInstanceData::OnObjectsReinstanced);
	}
#endif

	FControlRigAssetStrongReference CRAssetReference = GetTargetAssetReference(SharedData);
	if (CRAssetReference.IsValid())
	{
		if (UObject* AnimContext = GetAnimContext(Context))
		{
			USkeletalMeshComponent* BindableObject = const_cast<USkeletalMeshComponent*>(FInstanceData::GetBindableObject(Context));
			if (ensure(CreateControlRig(GetAnimContext(Context), BindableObject, SharedData->GetControlRigAssetReference(), this)))
			{
				FAnimNextControlRigTaskParams Params;
				Params.ControlRig = ControlRig;
				Params.bResetInputPoseToInitial = SharedData->bResetInputPoseToInitial;
				Params.bTransferInputPose = SharedData->bTransferInputPose;
				Params.bTransferInputCurves = SharedData->bTransferInputCurves;
				Params.bSetRefPoseFromSkeleton = SharedData->bSetRefPoseFromSkeleton;
				Params.bTransferPoseInGlobalSpace = SharedData->bTransferPoseInGlobalSpace;
				Params.EventQueue = SharedData->EventQueue;
				Params.InputMapping = SharedData->InputMapping;
				Params.OutputMapping = SharedData->OutputMapping;
#if UE_ENABLE_DEBUG_DRAWING
				Params.DebugDrawInterface = Context.GetDebugDrawInterface();
#endif

				EvaluationTask = MakeShared<FAnimNextControlRigTask>(MoveTemp(Params));

				InitializeControlRig(Context, Binding);
			}
		}
	}
}

template<typename TSharedData>
void TControlRigTraitBase<TSharedData>::FInstanceData::Destruct(const FExecutionContext& Context, const FTraitBinding& Binding)
{
	FTrait::FInstanceData::Destruct(Context, Binding);

#if WITH_EDITOR
	UE::UAF::ControlRig::FAnimNextControlRigModule::OnObjectsReinstanced.Remove(OnObjectsReinstancedHandle);
#endif

	EvaluationTask.Reset();

	DestroyControlRig(Context, Binding);
}

template<typename TSharedData>
void TControlRigTraitBase<TSharedData>::FInstanceData::InitializeControlRig(const FExecutionContext& Context, const FTraitBinding& Binding)
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	check(EvaluationTask);
	if (ControlRig)
	{
		EvaluationTask->Initialize(true, &PropertyMappings);

		ControlRig->OnInitialized_AnyThread().RemoveAll(this);
		OnInitializedHandle = ControlRig->OnInitialized_AnyThread().AddRaw(this, &TControlRigTraitBase<TSharedData>::FInstanceData::HandleOnInitialized_AnyThread);
	}
}

template<typename TSharedData>
void TControlRigTraitBase<TSharedData>::FInstanceData::DestroyControlRig(const FExecutionContext& Context, const FTraitBinding& Binding)
{
	if (ControlRig.Get() != nullptr)
	{
		if (OnInitializedHandle.IsValid())
		{
			ControlRig->OnInitialized_AnyThread().Remove(OnInitializedHandle);
			OnInitializedHandle.Reset();
		}
		ControlRig->MarkAsGarbage();
		ControlRig = nullptr;
	}
}

template<typename TSharedData>
UObject* TControlRigTraitBase<TSharedData>::FInstanceData::GetAnimContext(const FExecutionContext& Context)
{
	UObject* AnimContext = nullptr;

	if (const FAnimNextModuleInstance* ModuleInstance = Context.GetRootGraphInstance().GetModuleInstance())
	{
		AnimContext = ModuleInstance->GetObject();
	}

	return AnimContext;
}

template<typename TSharedData>
const USkeletalMeshComponent* TControlRigTraitBase<TSharedData>::FInstanceData::GetBindableObject(const FExecutionContext& Context)
{
	const USkeletalMeshComponent* BindableObject = Context.GetBindingObject().Get();
	
	return BindableObject;
}

template<typename TSharedData>
FControlRigAssetStrongReference TControlRigTraitBase<TSharedData>::FInstanceData::GetTargetAssetReference(const FSharedData* SharedData)
{
	if (SharedData != nullptr)
	{
		return SharedData->GetControlRigAssetReference();
	}
	return nullptr;
}

template<typename TSharedData>
int32 TControlRigTraitBase<TSharedData>::FInstanceData::GetExposedVariablesData(const UE::UAF::FTraitBinding& Binding, const FSharedData* SharedData, FControlRigVariableMappings::FCustomPropertyMappings& OutMappings)
{
	int32 NumElementsAdded = 0;

	if (SharedData && SharedData->HasValidControlRigReference() && SharedData->ExposedPropertyVariableNames.Num() > 0)
	{
		FControlRigAssetStrongReference CRAsset = SharedData->GetControlRigAssetReference();
		const int32 NumLatentProperties = SharedData->LatentProperties.Num();
		if (ensure(NumLatentProperties == SharedData->LatentPropertyMemoryLayouts.Num()))
		{
			const TArray<FRigVMExternalVariable> PublicVariables = CRAsset.GetExternalVariables();
			for (int32 LatentPropertyIndex = 0; LatentPropertyIndex < NumLatentProperties; LatentPropertyIndex++)
			{
				const FName& LatentPropertyName = SharedData->LatentProperties[LatentPropertyIndex];

				if (!SharedData->ExposedPropertyVariableNames.Contains(LatentPropertyName))	// Only process exposed public variables
				{
					continue;
				}

				const FRigVMExternalVariable* Variable = PublicVariables.FindByPredicate([&LatentPropertyName](const FRigVMExternalVariable& In)
					{
						return In.GetName() == LatentPropertyName;
					});

				if (Variable != nullptr)
				{
					uint32 PropertyAlignment = 0;
					uint32 PropertySize = 0;
					UE::UAF::TControlRigTraitBase<TSharedData>::GetVariableSizeAndAlignment(*Variable, PropertySize, PropertyAlignment);
					check(PropertyAlignment < MAX_uint16);
					check(PropertySize < MAX_uint16);

					if (ensure(SharedData->LatentPropertyMemoryLayouts[LatentPropertyIndex] == ((PropertySize << 16) | PropertyAlignment)))
					{
						const UE::UAF::FLatentPropertyHandle* TraitLatentPropertyHandles = Binding.GetLatentPropertyHandles();

						const UE::UAF::FLatentPropertyHandle& LatentPropertyHandle = TraitLatentPropertyHandles[LatentPropertyIndex];
						if (LatentPropertyHandle.IsOffsetValid())
						{
							const uint8* LatentPropertyMemory = const_cast<uint8*>(Binding.GetLatentProperty<uint8>(LatentPropertyHandle));
							OutMappings.AddVariable(Variable->GetName(), const_cast<uint8*>(Variable->GetMemory()), Variable->GetProperty(), LatentPropertyMemory);
						}
					}
				}
			}
		}
	}

	return NumElementsAdded;
}

template<typename TSharedData>
int32 TControlRigTraitBase<TSharedData>::FInstanceData::GetExposedControlsData(const UE::UAF::FTraitBinding& Binding, const FSharedData* SharedData, FControlRigVariableMappings::FCustomPropertyMappings& OutMappings)
{
	int32 NumElementsAdded = 0;

	if (SharedData && SharedData->HasValidControlRigReference())
	{
		const int32 NumLatentProperties = SharedData->LatentProperties.Num();
		if (ensure(NumLatentProperties == SharedData->LatentPropertyMemoryLayouts.Num()))
		{
			for (int32 LatentPropertyIndex = 0; LatentPropertyIndex < NumLatentProperties; LatentPropertyIndex++)
			{
				const FName& LatentPropertyName = SharedData->LatentProperties[LatentPropertyIndex];

				const int32 ControlIndex = SharedData->ExposedPropertyControlNames.IndexOfByKey(LatentPropertyName);
				if (ControlIndex == INDEX_NONE)	// Only process exposed controls
				{
					continue;
				}

				// Note : I can not check here if the controls exist, as I would have to instantiate the rig
				//        Using the exposed data to fill the information
				if (ensure(SharedData->ExposedPropertyControlNames.Num() == SharedData->ExposedPropertyControlTypes.Num()))
				{
					const FName ControlName = SharedData->ExposedPropertyControlNames[ControlIndex];
					const ERigControlType ControlType = SharedData->ExposedPropertyControlTypes[ControlIndex];

					uint32 PropertyAlignment = 0;
					uint32 PropertySize = 0;
					UE::UAF::TControlRigTraitBase<TSharedData>::GetControlSizeAndAlignment(ControlType, PropertySize, PropertyAlignment);
					ensure(PropertyAlignment < MAX_uint16);
					ensure(PropertySize < MAX_uint16);

					if (ensure(SharedData->LatentPropertyMemoryLayouts[LatentPropertyIndex] == ((PropertySize << 16) | PropertyAlignment)))
					{
						const UE::UAF::FLatentPropertyHandle* TraitLatentPropertyHandles = Binding.GetLatentPropertyHandles();

						const UE::UAF::FLatentPropertyHandle& LatentPropertyHandle = TraitLatentPropertyHandles[LatentPropertyIndex];
						if (LatentPropertyHandle.IsOffsetValid())
						{
							const uint8* LatentPropertyMemory = const_cast<uint8*>(Binding.GetLatentProperty<uint8>(LatentPropertyHandle));
							OutMappings.AddControl(ControlType, ControlName, nullptr, nullptr, LatentPropertyMemory);
						}
					}
				}
			}
		}
	}

	return NumElementsAdded;
}

template<typename TSharedData>
void TControlRigTraitBase<TSharedData>::FInstanceData::MapUAFVariablesToControlRig(const UE::UAF::FTraitBinding& Binding, const FSharedData* SharedData,
	FControlRigVariableMappings::FCustomPropertyMappings& OutMappings)
{
	FUAFInstanceVariableData& Variables = Binding.GetTraitPtr().GetNodeInstance()->GetOwner().GetVariables();

	FControlRigAssetStrongReference CRAssetReference = SharedData->GetControlRigAssetReference();
	if (const IRigVMRuntimeAssetInterface* ControlRigAsset = Cast<const IRigVMRuntimeAssetInterface>(CRAssetReference.Get()))
	{
		// Todo: this is a copy so its slow
		// Ideally we can get the details of the control rig without a copy
		TArray<FRigVMExternalVariable> ControlRigVariables = const_cast<IRigVMRuntimeAssetInterface*>(ControlRigAsset)->GetExternalVariables();

		for(FRigVMExternalVariable& ControlRigVar: ControlRigVariables)
		{
			// Check that this property isn't already in the map
			if (OutMappings.GetMappings().ContainsByPredicate([&ControlRigVar](const FControlRigVariableMappings::FCustomPropertyData& Candidate)
				{
					 return Candidate.TargetName == ControlRigVar.GetName();
				}))
			{
				// Mapping already exists, skip
				// This ensures latent properties are not overridden
				continue;
			}

			// Can happen if the rig has changed and hasn't been compiled
			if (ControlRigVar.GetMemory() == nullptr)
			{
				continue;
			}
			
			// Look up the variable in the uaf variable list
			FAnimNextVariableReference AnimNextReference = ControlRig::GetAnimNextVariableReferenceFromRigVMExternalVariable(ControlRigVar, ControlRigAsset);

			Variables.AccessVariableProperty(AnimNextReference, [&OutMappings, &ControlRigVar](const FProperty* Property, TArrayView<uint8> Data)
				{
					// Verify types match
					if (ControlRigVar.GetProperty()->SameType(Property))
					{
						OutMappings.AddVariable(ControlRigVar.GetName(), ControlRigVar.GetMemory(), Property, Data.GetData());
					}
					else
					{
						UE_LOGF(LogAnimation, Error, "Control rig property type %ls does not match UAF property type %ls for variable %ls", *ControlRigVar.GetProperty()->GetClass()->GetName(), *Property->GetClass()->GetName(), *ControlRigVar.GetName().ToString());
					}
				});
		}
	}
	
}

template<typename TSharedData>
void TControlRigTraitBase<TSharedData>::FInstanceData::HandleOnInitialized_AnyThread(URigVMHost*, const FName&)
{
	check(EvaluationTask);
	EvaluationTask->OnControlRigInitialized();

	#if WITH_EDITOR
		bRegenerateVariableMappings = true;	// required as FRigVMEditorModule::PreChange (UUSerStructs) recreates VM memory and requests a re-init, which recreates controls
	#endif
}

template<typename TSharedData>
void TControlRigTraitBase<TSharedData>::FInstanceData::InitializeCustomProperties(const FTraitBinding& Binding, const FSharedData* SharedData)
{
	check(EvaluationTask);
	// Setup mappings using the latent pin memory as source (we have to copy from latent pin to external variable / rig control)
	EvaluationTask->GetControlRigVariableMappings().InitializeCustomProperties(ControlRig, PropertyMappings);
}

#if WITH_EDITOR
template<typename TSharedData>
void TControlRigTraitBase<TSharedData>::FInstanceData::OnObjectsReinstanced(const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	if (ControlRig)
	{
		for (const TPair<UObject*, UObject*>& Pair : OldToNewInstanceMap)
		{
			const UObject* NewObject = Pair.Value;
			if((NewObject == nullptr) || (NewObject->GetOuter() != ControlRig->GetOuter()) || (!NewObject->IsA<UControlRig>()))
			{
				continue;
			}

			if (NewObject->GetClass() == ControlRig->GetClass())
			{
				bRefreshBindableObject = true;
				bReinitializeControlRig = true;
				break;
			}
		}
	}
}
#endif

//-------------------------
// --- FControlRigTrait ---
//-------------------------
// --- Custom Latent Pin Support impl ---
template<typename TSharedData>
void TControlRigTraitBase<TSharedData>::SerializeTraitSharedData(FArchive& Ar, FAnimNextTraitSharedData& SharedData) const
{
	TSharedData& ControlRigSharedData = static_cast<TSharedData&>(SharedData);

	TraitSuper::SerializeTraitSharedData(Ar, SharedData);

	if (Ar.IsLoading())
	{
		// Compute our latent property data based on our ControlRig class
		if (ControlRigSharedData.HasValidControlRigReference())
		{
			// We build the size/alignment map for each property even if their pin isn't hooked to anything
			// since handles are reserved for every one of them
			const int32 NumLatentProperties = ControlRigSharedData.ExposedPropertyVariableNames.Num() + ControlRigSharedData.ExposedPropertyControlNames.Num();
			ControlRigSharedData.LatentProperties.Reserve(NumLatentProperties);
			ControlRigSharedData.LatentPropertyMemoryLayouts.Reserve(NumLatentProperties);

			if (ControlRigSharedData.ExposedPropertyVariableNames.Num() > 0)
			{
				const TArray<FRigVMExternalVariable> PublicVariables = ControlRigSharedData.GetControlRigAssetReference().GetExternalVariables();
				if (!ensureMsgf(PublicVariables.Num() <= (int32)GetNumLatentTraitProperties(), TEXT("The ControlRig Trait only supports up to %u input variables"), GetNumLatentTraitProperties()))
				{
					return;
				}

				for (const FRigVMExternalVariable& Variable : PublicVariables)
				{
					if (!ControlRigSharedData.ExposedPropertyVariableNames.Contains(Variable.GetName()))	// Only process exposed public variables
					{
						continue;
					}

					uint32 PropertyAlignment = 0;
					uint32 PropertySize = 0;
					if (GetVariableSizeAndAlignment(Variable, PropertySize, PropertyAlignment))
					{
						check(PropertyAlignment < MAX_uint16);
						check(PropertySize < MAX_uint16);

						ControlRigSharedData.LatentProperties.Add(Variable.GetName());
						ControlRigSharedData.LatentPropertyMemoryLayouts.Add((PropertySize << 16) | PropertyAlignment);
					}
				}
			}

			if (ControlRigSharedData.ExposedPropertyControlNames.Num() > 0)
			{
				// Here I can not get the controls list, so I just use the exposed control names and types
				const int32 NumControls = ControlRigSharedData.ExposedPropertyControlNames.Num();
				check(NumControls == ControlRigSharedData.ExposedPropertyControlTypes.Num());

				for (int32 i = 0; i < NumControls; ++i)
				{
					uint32 PropertyAlignment = 0;
					uint32 PropertySize = 0;
					if (GetControlSizeAndAlignment(ControlRigSharedData.ExposedPropertyControlTypes[i], PropertySize, PropertyAlignment))
					{
						check(PropertyAlignment < MAX_uint16);
						check(PropertySize < MAX_uint16);

						ControlRigSharedData.LatentProperties.Add(ControlRigSharedData.ExposedPropertyControlNames[i]);
						ControlRigSharedData.LatentPropertyMemoryLayouts.Add((PropertySize << 16) | PropertyAlignment);
					}
				}
			}
		}
	}
}

template<typename TSharedData>
uint32 TControlRigTraitBase<TSharedData>::GetNumLatentTraitProperties() const
{
	// Number of latent trait properties must be known ahead of time to reserve space
	// We support a maximum number of input properties, each one will need a 2-byte handle in the shared data for each trait
	return 64;
}

template<typename TSharedData>
FTraitLatentPropertyMemoryLayout TControlRigTraitBase<TSharedData>::GetLatentPropertyMemoryLayout(const FAnimNextTraitSharedData& SharedData, FName PropertyName, uint32 PropertyIndex) const
{
	const TSharedData& ControlRigSharedData = static_cast<const TSharedData&>(SharedData);

	const int32 LatentPropertyIndex = ControlRigSharedData.LatentProperties.IndexOfByKey(PropertyName);
	if (LatentPropertyIndex == INDEX_NONE)
	{
		// This property isn't being tracked, ignore it
		return FTraitLatentPropertyMemoryLayout();
	}

	const uint32 PropertyLayout = ControlRigSharedData.LatentPropertyMemoryLayouts[LatentPropertyIndex];
	const uint32 PropertySize = PropertyLayout >> 16;
	const uint32 PropertyAlignment = PropertyLayout & MAX_uint16;

	return FTraitLatentPropertyMemoryLayout{ PropertySize, PropertyAlignment, LatentPropertyIndex };
}

// --- IUpdate impl --- 
template<typename TSharedData>
void TControlRigTraitBase<TSharedData>::OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	check(SharedData);

	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
	check(InstanceData);
	if (ensure(InstanceData->EvaluationTask))
	{
		InstanceData->EvaluationTask->RequestInit();
	}
}

template<typename TSharedData>
void TControlRigTraitBase<TSharedData>::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
{
	IUpdate::PreUpdate(Context, Binding, TraitState);

	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

	if (!InstanceData->Input.IsValid())
	{
		InstanceData->Input = Context.AllocateNodeInstance(Binding, SharedData->Input);
	}

#if WITH_EDITOR
	if (InstanceData->bRefreshBindableObject) // AnimNext full recompile Thaw function does not set a binding object
	{
		UObject* AnimContext = InstanceData->GetAnimContext(Context);
		USkeletalMeshComponent* BindableObject = const_cast<USkeletalMeshComponent*>(FInstanceData::GetBindableObject(Context));
		InstanceData->bRefreshBindableObject = !SetBindableObject(InstanceData->ControlRig, AnimContext, BindableObject);
	}
	if (InstanceData->bReinitializeControlRig)
	{
		InstanceData->PropertyMappings.Reset();
		UE::UAF::TControlRigTraitBase<TSharedData>::FInstanceData::GetExposedVariablesData(Binding, SharedData, InstanceData->PropertyMappings);
		UE::UAF::TControlRigTraitBase<TSharedData>::FInstanceData::GetExposedControlsData(Binding, SharedData, InstanceData->PropertyMappings);
		UE::UAF::TControlRigTraitBase<TSharedData>::FInstanceData::MapUAFVariablesToControlRig(Binding, SharedData, InstanceData->PropertyMappings);

		InstanceData->InitializeControlRig(Context, Binding);
		InstanceData->bReinitializeControlRig = false;
	}
#endif

	if (UControlRig* ControlRig = GetControlRig(InstanceData))
	{
#if WITH_EDITOR
		if (InstanceData->bRegenerateVariableMappings)
		{
			InstanceData->InitializeCustomProperties(Binding, SharedData);
			InstanceData->bRegenerateVariableMappings = false;
		}
#endif

		const float DeltaTime = TraitState.GetDeltaTime();
		ControlRig->SetDeltaTime(DeltaTime);

		InstanceData->EvaluationTask->GetControlRigVariableMappings().PropagateCustomInputProperties(ControlRig);
	}
}

// --- IEvaluate impl --- 
template<typename TSharedData>
void TControlRigTraitBase<TSharedData>::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
{
	const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
	FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
	check(InstanceData);
	if (ensure(InstanceData->EvaluationTask))
	{
		if (UControlRig* ControlRig = GetControlRig(InstanceData))
		{
			// The transform is used by the task to "transform" the debug drawings inside Control Rig
#if UE_ENABLE_DEBUG_DRAWING
			if (const USkeletalMeshComponent* BindableObject = FInstanceData::GetBindableObject(Context))
			{
				InstanceData->EvaluationTask->SetComponentTransform(BindableObject->GetComponentTransform());
			}
#endif

			Context.AppendTaskPtr(InstanceData->EvaluationTask);
		}
	}
}

// --- IHierarchy impl --- 
template<typename TSharedData>
uint32 TControlRigTraitBase<TSharedData>::GetNumChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding) const
{
	return 1;
}

template<typename TSharedData>
void TControlRigTraitBase<TSharedData>::GetChildren(const FExecutionContext& Context, const TTraitBinding<IHierarchy>& Binding, FChildrenArray& Children) const
{
	const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
	Children.Add(InstanceData->Input);
}

// --- IGarbageCollection impl --- 
template<typename TSharedData>
void TControlRigTraitBase<TSharedData>::AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const
{
	IGarbageCollection::AddReferencedObjects(Context, Binding, Collector);

	if (FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>())
	{
		if (InstanceData->ControlRig.Get() != nullptr)
		{
			Collector.AddReferencedObject(InstanceData->ControlRig);
		}
	}
}

// --- Utility --- 

template<typename TSharedData>
UControlRig* TControlRigTraitBase<TSharedData>::GetControlRig(FInstanceData* InstanceData)
{
	if (InstanceData)
	{
		return InstanceData->ControlRig.Get();
	}

	return nullptr;
}

template<typename TSharedData>
bool TControlRigTraitBase<TSharedData>::CreateControlRig(UObject* InAnimContext, UObject* InBindableObject, FControlRigAssetStrongReference InControlRigAssetReference, FInstanceData* InstanceData)
{
	if (InstanceData->ControlRig == nullptr && InControlRigAssetReference.IsValid())
	{
		bool bSuccess = false;
		// Let's make sure the GC isn't running when we try to create a new Control Rig 
		// or when setting the bindable object as GetDataSourceRegistry might create a new object
		{
			FGCScopeGuard GCGuard;
			InstanceData->ControlRig = InControlRigAssetReference.CreateInstance(InAnimContext);
			InstanceData->ControlRig->AtomicallyClearInternalFlags(EInternalObjectFlags::Async);
			
			bSuccess = SetBindableObject(InstanceData->ControlRig, InAnimContext, InBindableObject);
		}

#if WITH_EDITOR
		InstanceData->bRefreshBindableObject = !bSuccess;
#endif
	}

	return InstanceData->ControlRig != nullptr;
}


template<typename TSharedData>
bool TControlRigTraitBase<TSharedData>::SetBindableObject(UControlRig* ControlRig, UObject* InAnimContext, UObject* InBindableObject)
{
	if (ControlRig != nullptr && ensure(InAnimContext) && InBindableObject)
	{
		ControlRig->SetObjectBinding(MakeShared<FControlRigObjectBinding>());

		UObject* ObjectToBind = InBindableObject != nullptr ? InBindableObject : FControlRigObjectBinding::GetBindableObject(InAnimContext);
		check(ObjectToBind != nullptr);

		ControlRig->GetObjectBinding()->BindToObject(ObjectToBind);

		// register bindable object as data source (used for To World / From World transformations)
		ControlRig->GetDataSourceRegistry()->RegisterDataSource(UControlRig::OwnerComponent, ObjectToBind);

		return true;
	}
	return false;
}

#if WITH_EDITOR
template<typename TSharedData>
void TControlRigTraitBase<TSharedData>::GetProgrammaticPins(FAnimNextTraitSharedData* InSharedData, URigVMController* InController, int32 InParentPinIndex, const URigVMPin* InTraitPin, const FString& InDefaultValue, struct FRigVMPinInfoArray& OutPinArray) const
{
	if (TSharedData* SharedData = (TSharedData*)InSharedData)
	{
		if (SharedData->HasValidControlRigReference())
		{
			// --- Exposed Public Variables ---
			if (SharedData->ExposedPropertyVariableNames.Num() > 0)
			{
				const TArray<FRigVMExternalVariable> PublicVariables = SharedData->GetControlRigAssetReference().GetExternalVariables();

				for (const FRigVMExternalVariable& Variable : PublicVariables)
				{
					if (!SharedData->ExposedPropertyVariableNames.Contains(Variable.GetName()))	// Only process exposed public variables
					{
						continue;
					}

					if (Variable.GetMemory() == nullptr) // if we make a variable public but don't recompile, we have to skip, as it comes without memory
					{
						continue;
					}

					if (const URigVMPin* SubPin = InTraitPin->FindSubPin(Variable.GetName().ToString()))
					{
						const FString PinDefaultValue = SubPin->GetDefaultValue();
						FRigVMMemoryStorageStruct StorageDefaultValue(ERigVMMemoryType::External, {FRigVMPropertyDescription(Variable.GetProperty(), PinDefaultValue, Variable.GetName())});

						const uint8* DefaultMemory = StorageDefaultValue.GetDataByName<const uint8>(Variable.GetName());
						
						OutPinArray.AddPin(const_cast<FProperty*>(Variable.GetProperty()), InController, ERigVMPinDirection::Input, InParentPinIndex, ERigVMPinDefaultValueType::AutoDetect, DefaultMemory, true);
					}
					else
					{
						OutPinArray.AddPin(const_cast<FProperty*>(Variable.GetProperty()), InController, ERigVMPinDirection::Input, InParentPinIndex, ERigVMPinDefaultValueType::AutoDetect, Variable.GetMemory(), true);
					}
				}
			}

			// --- Exposed Controls ---
			const int32 NumExposedControls = SharedData->ExposedPropertyControlNames.Num();
			if (NumExposedControls > 0)
			{
				if (NumExposedControls == SharedData->ExposedPropertyControlDefaultValues.Num() && NumExposedControls == SharedData->ExposedPropertyControlTypes.Num())
				{
					for (int32 ControlIndex = 0; ControlIndex < NumExposedControls; ControlIndex++)
					{
						const FName& ControlName = SharedData->ExposedPropertyControlNames[ControlIndex];
						const FString& ControlDefaultValue = SharedData->ExposedPropertyControlDefaultValues[ControlIndex];
						const TRigVMTypeIndex TypeIndex = RigVMTypeUtils::TypeIndexFromPinType(URigHierarchy::GetControlPinType(SharedData->ExposedPropertyControlTypes[ControlIndex]));

						if (ensure(TypeIndex != INDEX_NONE))
						{
							OutPinArray.AddPin(InController, InParentPinIndex, ControlName, ERigVMPinDirection::Input, TypeIndex, ControlDefaultValue, ERigVMPinDefaultValueType::AutoDetect, nullptr, nullptr, true);
						}
					}
				}
				else
				{
					if (USkeleton* PreviewSkeleton = SharedData->GetPreviewSkeleton())
					{
						// Obtain the controls from the RigControlsData helper. This will instantiate a rig using the provided class and cache the controls until the class changes
						const TArray<FControlRigIOMapping::FControlsInfo>& RigControls = RigControlsData.GetControls(SharedData->GetControlRigAssetReference(), PreviewSkeleton);

						for (const FControlRigIOMapping::FControlsInfo& Control : RigControls)
						{
							const FName& ControlName = Control.Name;
							if (!SharedData->ExposedPropertyControlNames.Contains(ControlName))	// Only process exposed controls
							{
								continue;
							}

							const FString& ControlDefaultValue = Control.DefaultValue;
							const TRigVMTypeIndex TypeIndex = RigVMTypeUtils::TypeIndexFromPinType(Control.PinType);
							if (ensure(TypeIndex != INDEX_NONE))
							{
								OutPinArray.AddPin(InController, InParentPinIndex, ControlName, ERigVMPinDirection::Input, TypeIndex, ControlDefaultValue, ERigVMPinDefaultValueType::AutoDetect, nullptr, nullptr, true);
							}
						}
					}
				}
			}
		}
	}
}

template<typename TSharedData>
int32 TControlRigTraitBase<TSharedData>::GetLatentPropertyIndex(const FAnimNextTraitSharedData& InSharedData, FName PropertyName) const
{
	const TSharedData& SharedData = (const TSharedData&)InSharedData;
	return SharedData.LatentProperties.IndexOfByKey(PropertyName);
}

template<typename TSharedData>
uint32 TControlRigTraitBase<TSharedData>::GetLatentPropertyHandles(
	const FAnimNextTraitSharedData* InSharedData,
	TArray<FLatentPropertyMetadata>& OutLatentPropertyHandles,
	bool bFilterEditorOnly,
	const TFunction<uint16(FName PropertyName)>& GetTraitLatentPropertyIndex) const
{
	// Get shared data latent properties
	uint32 NumLatentPinsAdded = Super::GetLatentPropertyHandles(InSharedData, OutLatentPropertyHandles, bFilterEditorOnly, GetTraitLatentPropertyIndex);

	// Generate Control Rig exposed pins
	if (TSharedData* SharedData = (TSharedData*)InSharedData)
	{
		if (SharedData->HasValidControlRigReference())
		{
			// --- Iterate over public variables --- 
			if (SharedData->ExposedPropertyVariableNames.Num() > 0)
			{
				const TArray<FRigVMExternalVariable> PublicVariables = SharedData->GetControlRigAssetReference().GetExternalVariables();
				for (const FRigVMExternalVariable& Variable : PublicVariables)
				{
					if (!SharedData->ExposedPropertyVariableNames.Contains(Variable.GetName()))	// Only process exposed public variables
					{
						continue;
					}

					const FProperty* Property = Variable.GetProperty();

					FLatentPropertyMetadata Metadata;
					Metadata.Name = Property->GetFName();
					Metadata.RigVMIndex = GetTraitLatentPropertyIndex(Property->GetFName());

					// Always false for now, we don't support freezing yet
					Metadata.bCanFreeze = false;

					OutLatentPropertyHandles.Add(Metadata);
					NumLatentPinsAdded++;
				}
			}

			// --- Iterate over exposed controls ---
			if (SharedData->ExposedPropertyControlNames.Num() > 0)
			{
				if (USkeleton* PreviewSkeleton = SharedData->GetPreviewSkeleton())
				{
					// Obtain the controls from the RigControlsData helper. This will instantiate a rig using the provided class and cache the controls until the class changes
					const TArray<FControlRigIOMapping::FControlsInfo>& RigControls = RigControlsData.GetControls(SharedData->GetControlRigAssetReference(), PreviewSkeleton);

					for (const FControlRigIOMapping::FControlsInfo& Control : RigControls)
					{
						const FName& ControlName = Control.Name;
						if (!SharedData->ExposedPropertyControlNames.Contains(ControlName))	// Only process exposed controls
						{
							continue;
						}

						FLatentPropertyMetadata Metadata;
						Metadata.Name = ControlName;
						Metadata.RigVMIndex = GetTraitLatentPropertyIndex(ControlName);

						// Always false for now, we don't support freezing yet
						Metadata.bCanFreeze = false;

						OutLatentPropertyHandles.Add(Metadata);
						NumLatentPinsAdded++;
					}
				}
			}
		}
	}

	return NumLatentPinsAdded;
}

#endif // WITH_EDITOR

template<typename TSharedData>
bool TControlRigTraitBase<TSharedData>::GetVariableSizeAndAlignment(const FRigVMExternalVariable& Variable, uint32& PropertySize, uint32& PropertyAlignment)
{
	bool bValidType = false;

	if (Variable.GetCPPTypeObject() != nullptr)
	{
		if (const UScriptStruct* Struct = Cast<UScriptStruct>(Variable.GetCPPTypeObject()))
		{
			PropertySize = Struct->GetStructureSize();
			PropertyAlignment = Struct->GetMinAlignment();
			bValidType = true;
		}
		else if (const UEnum* Enum = Cast<UEnum>(Variable.GetCPPTypeObject()))
		{
			PropertySize = Variable.GetProperty()->GetSize();
			PropertyAlignment = Variable.GetProperty()->GetMinAlignment();
			bValidType = true;
		}
		else if (const UClass* Class = Cast<UClass>(Variable.GetCPPTypeObject()))
		{
			PropertySize = Class->GetStructureSize();
			PropertyAlignment = Class->GetMinAlignment();
			bValidType = true;
		}
		else
		{
			ensureMsgf(false, TEXT("Unsupported ControlRig public variable type: %s"), *Variable.GetExtendedCPPType().ToString());
			PropertySize = Variable.GetProperty()->GetSize();
			PropertyAlignment = Variable.GetProperty()->GetMinAlignment();
		}
	}
	else
	{
		PropertySize = Variable.GetProperty()->GetSize();
		PropertyAlignment = Variable.GetProperty()->GetMinAlignment();
		bValidType = true;
	}

	return bValidType;
}

template<typename TSharedData>
bool TControlRigTraitBase<TSharedData>::GetControlSizeAndAlignment(ERigControlType ControlType, uint32& PropertySize, uint32& PropertyAlignment)
{
	switch (ControlType)
	{
		case ERigControlType::Bool:
		{
			PropertySize = sizeof(bool);
			PropertyAlignment = PropertySize;
			return true;
		}
		case ERigControlType::Float:
		case ERigControlType::ScaleFloat:
		{
			PropertySize = sizeof(float);
			PropertyAlignment = PropertySize;
			return true;
		}
		case ERigControlType::Integer:
		{
			PropertySize = sizeof(int32);
			PropertyAlignment = PropertySize;
			return true;
		}
		case ERigControlType::Vector2D:
		{
			if (const UScriptStruct* Struct = TBaseStructure<FVector2D>::Get())
			{
				PropertySize = Struct->GetStructureSize();
				PropertyAlignment = Struct->GetMinAlignment();
				return true;
			}
		}
		case ERigControlType::Position:
		case ERigControlType::Scale:
		{
			if (const UScriptStruct* Struct = TBaseStructure<FVector>::Get())
			{
				PropertySize = Struct->GetStructureSize();
				PropertyAlignment = Struct->GetMinAlignment();
				return true;
			}
		}
		case ERigControlType::Rotator:
		{
			if (const UScriptStruct* Struct = TBaseStructure<FRotator>::Get())
			{
				PropertySize = Struct->GetStructureSize();
				PropertyAlignment = Struct->GetMinAlignment();
				return true;
			}
			break;
		}
		case ERigControlType::Transform:
		case ERigControlType::TransformNoScale:
		case ERigControlType::EulerTransform:
		{
			if (const UScriptStruct* Struct = TBaseStructure<FTransform>::Get())
			{
				PropertySize = Struct->GetStructureSize();
				PropertyAlignment = Struct->GetMinAlignment();
				return true;
			}
			break;
		}
		default:
		{
			ensureMsgf(false, TEXT("Unsupported ControlRig control type."));
			break;
		}
	}
	return false;
}

} // end namespace UE::UAF
