// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFControlRigAssetData.h"

#include "ControlRig.h"

bool FUAFGraphFactoryAsset_ControlRig::Validate() const
{
	return ControlRigAssetReference.IsValid();
}

void FUAFGraphFactoryAsset_ControlRig::GetObjectReferences(TArray<const UObject*>& OutReferencedObjects) const
{
	OutReferencedObjects.Add(ControlRigAssetReference.Get());
}

#if WITH_EDITORONLY_DATA
void FUAFGraphFactoryAsset_ControlRig::PostSerialize(const FArchive& Ar)
{
	if (ControlRigClass_DEPRECATED)
	{
		ControlRigAssetReference.Set(ControlRigClass_DEPRECATED);
		ControlRigClass_DEPRECATED = nullptr;
	}
}
#endif
