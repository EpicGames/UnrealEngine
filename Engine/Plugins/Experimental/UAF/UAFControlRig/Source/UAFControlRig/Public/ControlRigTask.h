// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EvaluationVM/EvaluationTask.h"
#include "Animation/AnimCurveTypes.h"
#include "LODPose.h"
#include "AnimNextStats.h"
#include "ControlRig.h"
#include "Delegates/IDelegateInstance.h"
#include "AnimNextControlRigHierarchyMappings.h"
#include "Tools/ControlRigVariableMappings.h"
#include "Tools/ControlRigIOSettings.h"
#if WITH_EDITOR
#include "ControlRigIOMapping.h"
#endif

// --- ---
#include "ControlRigTask.generated.h"

namespace UE::UAF
{
struct FKeyframeState;
}

DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF Task: ControlRig"), STAT_AnimNext_Task_ControlRig, STATGROUP_AnimNext, UAFCONTROLRIG_API);

USTRUCT()
struct FControlRigEventName
{
	GENERATED_BODY()

	FControlRigEventName()
		: EventName(NAME_None)
	{
	}

	UPROPERTY(EditAnywhere, Category = Links)
	FName EventName;
};

USTRUCT()
struct FAnimNextControlRigTaskParams
{
	GENERATED_BODY()

	bool bResetInputPoseToInitial = true;
	bool bTransferInputPose = true;
	bool bTransferInputCurves = true;
	bool bSetRefPoseFromSkeleton = false;
	bool bTransferPoseInGlobalSpace = false;
	TArray<FControlRigEventName> EventQueue;
	TMap<FName, FName> InputMapping;
	TMap<FName, FName> OutputMapping;

	FRigVMDrawInterface* DebugDrawInterface = nullptr;

	FControlRigVariableMappings ControlRigVariableMappings;
	mutable UE::UAF::ControlRig::FAnimNextControlRigHierarchyMappings ControlRigHierarchyMappings;

	TObjectPtr<UControlRig> ControlRig;

	uint16 LastBonesSerialNumberForCacheBones = 0;
	mutable bool bControlRigRequiresInitialization = false;
	mutable bool bUpdateInputOutputMapping = false;
	FTransform ComponentTransform;

	bool bConsumesPreviousPose = true;
};

USTRUCT()
struct UAFCONTROLRIG_API FAnimNextControlRigTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextControlRigTask)

	FAnimNextControlRigTask() = default;
	explicit FAnimNextControlRigTask(FAnimNextControlRigTaskParams&& InParams)
		: Params(MoveTemp(InParams))
	{
	}

	// Task entry point
	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

public:
	const FAnimNextControlRigTaskParams& GetParams() const { return Params; }

	// Initializes the control rig
	void Initialize(bool bInitControlRig, const FControlRigVariableMappings::FCustomPropertyMappings* PropertyMappings = nullptr);

	// Marks the control rig as needing initialization on the next execute
	void RequestInit();

	// Called when the control rig has been initialized
	void OnControlRigInitialized();

#if UE_ENABLE_DEBUG_DRAWING
	void SetComponentTransform(const FTransform& InComponentTransform);
#endif

	FControlRigVariableMappings& GetControlRigVariableMappings() { return Params.ControlRigVariableMappings; }
	const FControlRigVariableMappings& GetControlRigVariableMappings() const { return Params.ControlRigVariableMappings; }

	UE::UAF::ControlRig::FAnimNextControlRigHierarchyMappings& GetControlRigHierarchyMappings() { return Params.ControlRigHierarchyMappings; }
	const UE::UAF::ControlRig::FAnimNextControlRigHierarchyMappings& GetControlRigHierarchyMappings() const { return Params.ControlRigHierarchyMappings; }

	UControlRig* GetControlRig() const { return Params.ControlRig; }

private:
	FAnimNextControlRigTaskParams Params;

	FControlRigIOSettings InputSettings;
	FControlRigIOSettings OutputSettings;

	mutable int32 LastLOD = INDEX_NONE;

	bool bExecute = true;
	mutable bool bClearEventQueueRequired = false;
	void ExecuteControlRig(UE::UAF::FEvaluationVM& VM, UE::UAF::FKeyframeState& KeyFrameState) const;
	void UpdateInput(UE::UAF::FEvaluationVM& VM, UE::UAF::FKeyframeState& KeyFrameState, UE::UAF::FKeyframeState& InOutput) const;
	void UpdateOutput(UE::UAF::FEvaluationVM& VM, UE::UAF::FKeyframeState& KeyFrameState, UE::UAF::FKeyframeState& InOutput) const;

	bool CanExecute() const;

	void QueueControlRigDrawInstructions(FRigVMDrawInterface* DebugDrawInterface, const FTransform& ComponentTransform) const;



};
