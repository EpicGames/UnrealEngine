// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlRigAssetReference.h"
#include "CoreTypes.h"
#include "Templates/SubclassOf.h"
#include "UAF/UAFAssetData.h"

#include "UAFControlRigAssetData.generated.h"

class UControlRig;

USTRUCT(DisplayName="Control Rig Asset")
struct FUAFGraphFactoryAsset_ControlRig : public FUAFGraphFactoryAsset
{
	GENERATED_BODY()
	FUAFGraphFactoryAsset_ControlRig(const FControlRigAssetStrongReference& InControlRigReference) : ControlRigAssetReference(InControlRigReference) {}
	FUAFGraphFactoryAsset_ControlRig() = default;

	virtual bool Validate() const override;
	
	virtual void GetObjectReferences(TArray<const UObject*>& OutReferencedObjects) const override;

#if WITH_EDITORONLY_DATA
	/** The type of this animation ControlRig */
	UPROPERTY(meta=(DeprecatedProperty))
	TSubclassOf<UControlRig> ControlRigClass_DEPRECATED;
#endif
	
	/** The type of this animation ControlRig */
	UPROPERTY(EditAnywhere, Category = ControlRig)
	FControlRigAssetStrongReference ControlRigAssetReference;
	
#if WITH_EDITORONLY_DATA
	void PostSerialize(const FArchive& Ar);
#endif
};


#if WITH_EDITORONLY_DATA
template<>
struct TStructOpsTypeTraits<FUAFGraphFactoryAsset_ControlRig> : public TStructOpsTypeTraitsBase2<FUAFGraphFactoryAsset_ControlRig>
{
	enum
	{
		WithPostSerialize = true,
	};
};
#endif