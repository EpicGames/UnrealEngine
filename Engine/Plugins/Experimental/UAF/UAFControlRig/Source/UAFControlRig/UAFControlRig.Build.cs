// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class UAFControlRig : ModuleRules
	{
		public UAFControlRig(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"CoreUObject",
					"Engine",
					"Core",
					"ControlRig",
					"RigVM",
					"UAF",
					"UAFAnimGraph",
				}
			);

			if (Target.bBuildEditor == true)
			{
				PublicDependencyModuleNames.AddRange(
					new string[]
					{
						"ControlRigDeveloper"
					}
				);

				PrivateDependencyModuleNames.AddRange(
				   new string[]
				   {
						"WorkspaceEditor",
						"SlateCore",
						"Slate",
						"UAFUncookedOnly",
						"RigVMDeveloper",
						"AnimationCore",
				   });
			}
		}
	}
}
