// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyEditorModule.h"
#include "ControlRigIOMapping.h"

class URigVMPin;
class IDetailCategoryBuilder;
class UEdGraphNode;
class URigVMNode;
struct FRigVMVariableMappingInfo;
struct FControlRigTraitSharedDataBase;

namespace UE::Workspace
{
	class IWorkspaceEditor;
}

namespace UE::UAF::Editor
{

class FControlRigTraitSharedDataCustomization : public IPropertyTypeCustomization
{
public:
	FControlRigTraitSharedDataCustomization() = default;
	virtual ~FControlRigTraitSharedDataCustomization();

private:
	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	// --- IO Mapping ---
	void OnVariableMappingChanged(const FName& PathName, const FName& Curve, bool bInput);
	void OnPropertyExposeCheckboxChanged(ECheckBoxState NewState, FName PropertyName);

	static TSharedPtr<FStructOnScope> GetControlRigSharedData(const TSharedPtr<IPropertyHandle>& StructPropertyHandle);
	FControlRigAssetStrongReference GetTargetAssetReference() const;
	USkeleton* GetTargetSkeleton() const;

	static const FControlRigIOMapping::FControlsInfo* GetControlInfo(const TArray<FControlRigIOMapping::FControlsInfo>& Controls, const FName& ControlName);
	template<typename TSharedData>
	static bool IsVariableProperty(TSharedData* ControlRigTraitSharedData, const FName& PropertyName);

	void OnObjectsReinstanced(const TMap<UObject*, UObject*>& OldToNewInstanceMap);

	FString FindControlRigTraitPinName(URigVMNode* ModelNode);

	TSharedPtr<IPropertyHandle> StructPropertyHandle;
	TSharedPtr<FStructOnScope> ScopedControlRigTraitSharedData;

	TWeakObjectPtr<UEdGraphNode> SelectedNodeWeak = nullptr;
	TArray<FOptionalPinFromProperty> CustomPinProperties;
	TSharedPtr <FControlRigIOMapping> ControlRigIOMapping;

	FDelegateHandle OnObjectsReinstancedHandle;
};

}
