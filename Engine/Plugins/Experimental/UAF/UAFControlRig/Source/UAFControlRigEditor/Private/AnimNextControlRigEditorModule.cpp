// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextControlRigEditorModule.h"
#include "Modules/ModuleManager.h"
#include "RigVMCore/RigVMRegistry.h"
#include "ControlRigTraitCustomization.h"
#include "ControlRigTrait.h"
#include "ControlRig.h"
#include "IAnimNextEditorModule.h"
#include "AnimGraphUncookedOnlyUtils.h"
#include "AnimNextEdGraphNode.h"
#include "IWorkspaceEditorModule.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "UObject/Class.h"
#include "Editor/EditorEngine.h"
#include "K2Node.h"
#include "Animation/Skeleton.h"

extern UNREALED_API UEditorEngine* GEditor;

namespace UE::UAF::ControlRig::Editor
{

void FAnimNextControlRigEditorModule::StartupModule()
{
	static TPair<UClass*, FRigVMRegistry::ERegisterObjectOperation> const AllowedObjectTypes[] =
	{
		{ UControlRig::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::ClassAndChildren },
		{ UClass::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
		{ USkeleton::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
	};

	static UScriptStruct* const AllowedStructTypes[] =
	{
		FInputScaleBias::StaticStruct(),
		FInputAlphaBoolBlend::StaticStruct(),
	};

	FRigVMRegistry& RigVMRegistry = FRigVMRegistry::Get();
	RigVMRegistry.RegisterObjectTypes(AllowedObjectTypes);
	RigVMRegistry.RegisterStructTypes(AllowedStructTypes);

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	// This generates all the Details panel ControlRig specific data and enables programmatic pin generation
	PropertyModule.RegisterCustomPropertyTypeLayout(
		"ControlRigTraitSharedData",
		FOnGetPropertyTypeCustomizationInstance::CreateLambda([] { return MakeShared<UE::UAF::Editor::FControlRigTraitSharedDataCustomization>(); }));

	UE::UAF::Editor::IAnimNextEditorModule& AnimNextEditorModule = FModuleManager::LoadModuleChecked<UE::UAF::Editor::IAnimNextEditorModule>("UAFEditor");

	OnNodeDblClickDelegateHandle = AnimNextEditorModule.RegisterNodeDblClickHandler(UE::UAF::Editor::IAnimNextEditorModule::FNodeDblClickNotificationDelegate::CreateLambda([](const Workspace::FWorkspaceEditorContext& InContext, const UEdGraphNode* InNode)
		{
			if (const UAnimNextEdGraphNode* RigVMEdGraphNode = Cast<UAnimNextEdGraphNode>(InNode))
			{
				const URigVMNode* ModelNode = RigVMEdGraphNode->GetModelNode();

				if (UncookedOnly::FAnimGraphUtils::IsTraitStackNode(ModelNode))
				{
					static const FString ControlRigTraitName = FString(TEXT("FControlRigTrait"));
					static const FString ControlRigClassPinName = FString(TEXT("ControlRigClass"));

					const TArray<URigVMPin*> TraitPins = ModelNode->GetTraitPins();
					for (const URigVMPin* TraitPin : TraitPins)
					{
						if (TraitPin->GetName() == ControlRigTraitName)
						{
							for (const URigVMPin* SubPin : TraitPin->GetSubPins())
							{
								if (SubPin->GetName() == ControlRigClassPinName)
								{
									if (UClass* ResultClass = UClass::TryFindTypeSlow<UClass>(SubPin->GetDefaultValue()))
									{
										if (TSharedPtr<Workspace::IWorkspaceEditor> WorkspaceEditor = InContext.WorkspaceEditor)
										{
											// Control Rig is not integrated in AnimNext, so we open the whole BlueprintEditor for the asset
											GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ResultClass->ClassGeneratedBy);
											break;
										}
									}
								}
							}
						}
					}
				}
			}
		}));
}

void FAnimNextControlRigEditorModule::ShutdownModule()
{
	if (FModuleManager::Get().IsModuleLoaded("UAFEditor"))
	{
		UE::UAF::Editor::IAnimNextEditorModule& AnimNextEditorModule = FModuleManager::LoadModuleChecked<UE::UAF::Editor::IAnimNextEditorModule>("UAFEditor");

		AnimNextEditorModule.UnregisterNodeDblClickHandler(OnNodeDblClickDelegateHandle);
	}
}

} // end namespace

IMPLEMENT_MODULE(UE::UAF::ControlRig::Editor::FAnimNextControlRigEditorModule, UAFControlRigEditor)
