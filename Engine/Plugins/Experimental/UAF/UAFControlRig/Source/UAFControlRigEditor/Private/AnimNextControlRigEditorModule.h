// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

namespace UE::UAF::ControlRig::Editor
{

class FAnimNextControlRigEditorModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	FDelegateHandle OnNodeDblClickDelegateHandle;

};

} // end namespace
