// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigTraitCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "IWorkspaceEditor.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/SRigVMVariableMappingWidget.h"
#include "ControlRigTrait.h"
#include "RigVMBlueprintGeneratedClass.h"
#include "ControlRigBlueprintLegacy.h"
#include "Engine/SkeletalMesh.h"
#include "IPropertyUtilities.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "AnimGraphUncookedOnlyUtils.h"
#include "RigVMModel/RigVMController.h"
#include "AnimNextControlRigModule.h"
#include "ScopedTransaction.h"
#include "K2Node.h"
#include "TraitCore/TraitRegistry.h"
#include "RigVMStringUtils.h"
#include "Templates/UAFGraphNodeTemplate.h"

#define LOCTEXT_NAMESPACE "ControlRigTraitSharedDataCustomization"

namespace UE::UAF::Editor
{

FControlRigTraitSharedDataCustomization::~FControlRigTraitSharedDataCustomization()
{
	if (OnObjectsReinstancedHandle.IsValid())
	{
		UE::UAF::ControlRig::FAnimNextControlRigModule::OnObjectsReinstanced.Remove(OnObjectsReinstancedHandle);
	}
}

void FControlRigTraitSharedDataCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	if (TSharedPtr<IPropertyUtilities> PropertyUtils = InCustomizationUtils.GetPropertyUtilities())
	{
		const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = PropertyUtils->GetSelectedObjects();
		if (SelectedObjects.Num() != 1)
		{
			return;
		}

		SelectedNodeWeak = Cast<UEdGraphNode>(SelectedObjects[0]);
	}

	StructPropertyHandle = InPropertyHandle;
	ScopedControlRigTraitSharedData = GetControlRigSharedData(StructPropertyHandle);

	FControlRigTraitSharedDataBase* ControlRigTraitSharedData = (FControlRigTraitSharedDataBase*)ScopedControlRigTraitSharedData->GetStructMemory();
	if (!ControlRigTraitSharedData)
	{
		return;
	}

	CustomPinProperties.Reset(ControlRigTraitSharedData->ExposedPropertyVariableNames.Num() + ControlRigTraitSharedData->ExposedPropertyControlNames.Num());
	for (const FName& PropertyName : ControlRigTraitSharedData->ExposedPropertyVariableNames)
	{
		FOptionalPinFromProperty OptionalPin(PropertyName, true, true, FString(), FText(), false, NAME_None, false);
		OptionalPin.bIsOverrideEnabled = false;
		CustomPinProperties.Add(OptionalPin);
	}
	for (const FName& PropertyName : ControlRigTraitSharedData->ExposedPropertyControlNames)
	{
		FOptionalPinFromProperty OptionalPin(PropertyName, true, true, FString(), FText(), false, NAME_None, false);
		OptionalPin.bIsOverrideEnabled = false;
		CustomPinProperties.Add(OptionalPin);
	}
	ControlRigIOMapping = MakeShared<FControlRigIOMapping>(ControlRigTraitSharedData->InputMapping, ControlRigTraitSharedData->OutputMapping, CustomPinProperties);
	ControlRigIOMapping->GetOnPinCheckStateChangedDelegate().BindSP(this, &FControlRigTraitSharedDataCustomization::OnPropertyExposeCheckboxChanged);
	ControlRigIOMapping->GetOnVariableMappingChanged().BindSP(this, &FControlRigTraitSharedDataCustomization::OnVariableMappingChanged);
	ControlRigIOMapping->GetOnGetTargetSkeletonDelegate().BindSP(this, &FControlRigTraitSharedDataCustomization::GetTargetSkeleton);
	ControlRigIOMapping->GetOnGetTargetAssetReferenceDelegate().BindSP(this, &FControlRigTraitSharedDataCustomization::GetTargetAssetReference);
	ControlRigIOMapping->SetIgnoreVariablesWithNoMemory(true); // AnimBP creates it's own variables, AnimNext needs the memory to create the pin

	OnObjectsReinstancedHandle = UE::UAF::ControlRig::FAnimNextControlRigModule::OnObjectsReinstanced.AddRaw(this, &FControlRigTraitSharedDataCustomization::OnObjectsReinstanced);

	InHeaderRow.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	];
}

void FControlRigTraitSharedDataCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	// --- Add struct default members, as they would be without the customization ---
	uint32 NumChildren = 0;
	if (InPropertyHandle->GetNumChildren(NumChildren) == FPropertyAccess::Success)
	{
		for (uint32 Index = 0; Index < NumChildren; ++Index)
		{
			TSharedRef<IPropertyHandle> ChildPropertyHandle = InPropertyHandle->GetChildHandle(Index).ToSharedRef();

			InChildBuilder.AddProperty(ChildPropertyHandle);
		}
	}

	IDetailLayoutBuilder& DetailBuilder = InChildBuilder.GetParentCategory().GetParentLayout();

	if (ControlRigIOMapping.IsValid())
	{
		ControlRigIOMapping->CreateVariableMappingWidget(DetailBuilder);
	}
}

void FControlRigTraitSharedDataCustomization::OnPropertyExposeCheckboxChanged(ECheckBoxState NewState, FName PropertyName)
{
	if (FControlRigTraitSharedDataBase* ControlRigTraitSharedData = (FControlRigTraitSharedDataBase*)ScopedControlRigTraitSharedData->GetStructMemory())
	{
		const bool bInput = ControlRigIOMapping->IsInputProperty(PropertyName);
		ControlRigIOMapping->SetIOMapping(bInput, PropertyName, NAME_None);

		if (URigVMEdGraphNode* EdGraphNode = Cast<URigVMEdGraphNode>(SelectedNodeWeak.Get()))
		{
			if (URigVMNode* ModelNode = EdGraphNode->GetModelNode())
			{
				if (URigVMGraph* Model = EdGraphNode->GetModel())
				{
					URigVMController* Controller = EdGraphNode->GetController();

					if (URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(SelectedNodeWeak->GetGraph()))
					{
						static FName ExposedPropertyVariablesName = GET_MEMBER_NAME_CHECKED(FControlRigTraitSharedDataBase, ExposedPropertyVariableNames);
						static FName ExposedPropertyControlsName = GET_MEMBER_NAME_CHECKED(FControlRigTraitSharedDataBase, ExposedPropertyControlNames);
						static FName ExposedPropertyControlTypesName = GET_MEMBER_NAME_CHECKED(FControlRigTraitSharedDataBase, ExposedPropertyControlTypes);
						static FName ExposedPropertyControlDefaultValuesName = GET_MEMBER_NAME_CHECKED(FControlRigTraitSharedDataBase, ExposedPropertyControlDefaultValues);
						
						const FString ControlRigTraitName = FindControlRigTraitPinName(ModelNode);
						ensureAlwaysMsgf(!ControlRigTraitName.IsEmpty(), TEXT("Can not Find a Control Rig Trait pin by name at node : [%s]."), *ModelNode->GetFName().ToString());

						const FString ControlRigTraitPrefix(ControlRigTraitName + TEXT("."));
						const FString ExposedVariablesSubPath(ControlRigTraitPrefix + ExposedPropertyVariablesName.ToString());
						const FString ExposedControlsSubPath(ControlRigTraitPrefix + ExposedPropertyControlsName.ToString());
						const FString ExposedControlTypesSubPath(ControlRigTraitPrefix + ExposedPropertyControlTypesName.ToString());
						const FString ExposedControlDefaultValuesSubPath(ControlRigTraitPrefix + ExposedPropertyControlDefaultValuesName.ToString());

						static const FText ExposePropertyToPinText = LOCTEXT("ExposePropertyToPin", "Expose Property to Pin");
						static const FText RemovePropertyPinText = LOCTEXT("RemovePropertyOptionalPin", "Removed Property Optional Pin");
						const FText& TransactionText = NewState == ECheckBoxState::Checked ? ExposePropertyToPinText : RemovePropertyPinText;

						FRigVMControllerCompileBracketScope CompileScope(Controller); // avoid multiple recompilations and force a details refresh after recompile
						FScopedTransaction Transaction(TransactionText);
						Model->Modify();

						if (NewState == ECheckBoxState::Checked)
						{
							const bool bIsInput = ControlRigIOMapping->IsInputProperty(PropertyName);
							// if checked, we clear mapping and unclear all children
							ControlRigIOMapping->SetIOMapping(bIsInput, PropertyName, NAME_None);
						}

						URigVMPin* VariablesPin = ModelNode->FindPin(ExposedVariablesSubPath);
						URigVMPin* ControlsPin = ModelNode->FindPin(ExposedControlsSubPath);
						URigVMPin* ControlsTypesPin = ModelNode->FindPin(ExposedControlTypesSubPath);
						URigVMPin* ControlsDefaultValuesPin = ModelNode->FindPin(ExposedControlDefaultValuesSubPath);
						if (ensure(VariablesPin && ControlsPin && ControlsTypesPin && ControlsDefaultValuesPin))
						{
							ControlRigTraitSharedData->ExposedPropertyVariableNames.Reset(CustomPinProperties.Num());
							ControlRigTraitSharedData->ExposedPropertyControlNames.Reset(CustomPinProperties.Num());
							ControlRigTraitSharedData->ExposedPropertyControlTypes.Reset(CustomPinProperties.Num());
							ControlRigTraitSharedData->ExposedPropertyControlDefaultValues.Reset(CustomPinProperties.Num());

							for (const FOptionalPinFromProperty& OptionalPin : CustomPinProperties)
							{
								if (OptionalPin.bShowPin)
								{
									bool bIsVariableProperty = false;
									if (ScopedControlRigTraitSharedData->GetStruct() == FControlRigTraitSharedData::StaticStruct())
									{
										FControlRigTraitSharedData* SharedData = (FControlRigTraitSharedData*)ScopedControlRigTraitSharedData->GetStructMemory();
										bIsVariableProperty = IsVariableProperty(SharedData, OptionalPin.PropertyName);
									}
									else if (ScopedControlRigTraitSharedData->GetStruct() == FControlRigTraitSharedData_v2::StaticStruct())
									{
										FControlRigTraitSharedData_v2* SharedData = (FControlRigTraitSharedData_v2*)ScopedControlRigTraitSharedData->GetStructMemory();
										bIsVariableProperty = IsVariableProperty(SharedData, OptionalPin.PropertyName);
									}
									if (bIsVariableProperty)
									{
										ControlRigTraitSharedData->ExposedPropertyVariableNames.Add(OptionalPin.PropertyName);
									}
									else
									{
										const TArray<FControlRigIOMapping::FControlsInfo>& Controls = ControlRigIOMapping->GetControls();
										if (const FControlRigIOMapping::FControlsInfo* ControlInfo = GetControlInfo(Controls, OptionalPin.PropertyName))
										{
											ControlRigTraitSharedData->ExposedPropertyControlNames.Add(OptionalPin.PropertyName);
											ControlRigTraitSharedData->ExposedPropertyControlTypes.Add(ControlInfo->ControlType);
											ControlRigTraitSharedData->ExposedPropertyControlDefaultValues.Add(ControlInfo->DefaultValue);
										}
									}
								}
							}

							static constexpr const auto SetPinArrayDefaultValue = ([](URigVMController* Controller, URigVMPin* Pin, FControlRigTraitSharedDataBase* ControlRigTraitSharedData, const FName& PropertyName, void* ArrayData) -> bool
								{
									if (const FProperty* Property = ControlRigTraitSharedData->StaticStruct()->FindPropertyByName(PropertyName))
									{
										FString DefaultValue;
										Property->ExportText_Direct(DefaultValue, ArrayData, nullptr, nullptr, PPF_None);
										if (DefaultValue.IsEmpty())
										{
											static FString EmptyArray(TEXT("()"));
											DefaultValue = EmptyArray;
										}
										return Controller->SetPinDefaultValue(Pin->GetPinPath(), DefaultValue);
									}
									return false;
								});

							SetPinArrayDefaultValue(Controller, VariablesPin, ControlRigTraitSharedData, ExposedPropertyVariablesName, &ControlRigTraitSharedData->ExposedPropertyVariableNames);
							SetPinArrayDefaultValue(Controller, ControlsPin, ControlRigTraitSharedData, ExposedPropertyControlsName, &ControlRigTraitSharedData->ExposedPropertyControlNames);
							SetPinArrayDefaultValue(Controller, ControlsTypesPin, ControlRigTraitSharedData, ExposedPropertyControlTypesName, &ControlRigTraitSharedData->ExposedPropertyControlTypes);
							SetPinArrayDefaultValue(Controller, ControlsDefaultValuesPin, ControlRigTraitSharedData, ExposedPropertyControlDefaultValuesName, &ControlRigTraitSharedData->ExposedPropertyControlDefaultValues);
						}

						Controller->RepopulatePinsOnNode(ModelNode, true, false, true);

						// Now that pins have been added programmatically, we must also add them to the node layout or they will not be visible
						TArray<URigVMPin*> Pins = ModelNode->GetAllPinsRecursively();
						FRigVMNodeLayout Layout = ModelNode->GetNodeLayout();
						TArray<FString> PinPathsRequiringCategory;
						PinPathsRequiringCategory.Reserve(Pins.Num());
						for(URigVMPin* Pin : Pins)
						{
							if (Pin->GetIndexInCategory() == INDEX_NONE && Pin->GetDirection() != ERigVMPinDirection::Hidden)
							{
								FString PinPath = Pin->GetPinPath(false); // this contains the node name as prefix, but the node layout doesn't want that
								FString NodePrefix = ModelNode ->GetName() + ".";
								PinPath.RemoveFromStart(NodePrefix);
								PinPathsRequiringCategory.Add(PinPath);
							}
						}

						UUAFGraphNodeTemplate::SetCategoryForPinsInLayout(PinPathsRequiringCategory, TEXT("Control Rig"), Layout, true);
						Controller->SetNodeLayout(ModelNode->GetFName(), Layout);
					}
				}
			}
		}
	}
}

void FControlRigTraitSharedDataCustomization::OnVariableMappingChanged(const FName& PathName, const FName& Curve, bool bInput)
{
	if (URigVMEdGraphNode* EdGraphNode = Cast<URigVMEdGraphNode>(SelectedNodeWeak.Get()))
	{
		if (URigVMNode* ModelNode = EdGraphNode->GetModelNode())
		{
			if (URigVMGraph* Model = EdGraphNode->GetModel())
			{
				URigVMController* Controller = EdGraphNode->GetController();

				FRigVMControllerCompileBracketScope CompileScope(Controller); // avoid multiple recompilations and force a details refresh after recompile

				FScopedTransaction Transaction(LOCTEXT("VariableMappingChanged", "Change Variable Mapping"));
				Model->Modify();

				// @todo: this is not enough when we start breaking down struct
				ControlRigIOMapping->SetIOMapping(bInput, PathName, Curve);
			}
		}
	}
}

TSharedPtr<FStructOnScope> FControlRigTraitSharedDataCustomization::GetControlRigSharedData(const TSharedPtr<IPropertyHandle>& StructPropertyHandle)
{
	TSharedPtr<FStructOnScope> ControlRigSharedData;

	TArray<TSharedPtr<FStructOnScope>> OutStructs;
	StructPropertyHandle->GetOuterStructs(OutStructs);

	if (OutStructs.Num() == 1)
	{
		ControlRigSharedData = OutStructs[0];
	}

	return ControlRigSharedData;
}

FControlRigAssetStrongReference FControlRigTraitSharedDataCustomization::GetTargetAssetReference() const
{
	if (ScopedControlRigTraitSharedData->GetStruct() == FControlRigTraitSharedData::StaticStruct())
	{
		FControlRigTraitSharedData* ControlRigTraitSharedData = (FControlRigTraitSharedData*)ScopedControlRigTraitSharedData->GetStructMemory();
		return ControlRigTraitSharedData->GetControlRigAssetReference();
	}
	else if (ScopedControlRigTraitSharedData->GetStruct() == FControlRigTraitSharedData_v2::StaticStruct())
	{
		FControlRigTraitSharedData_v2* ControlRigTraitSharedData = (FControlRigTraitSharedData_v2*)ScopedControlRigTraitSharedData->GetStructMemory();
		return ControlRigTraitSharedData->GetControlRigAssetReference();
	}
	return FControlRigAssetStrongReference();
}

USkeleton* FControlRigTraitSharedDataCustomization::GetTargetSkeleton() const
{
	USkeleton* TargetSkeleton = nullptr;

	if (ScopedControlRigTraitSharedData->GetStruct() == FControlRigTraitSharedData::StaticStruct())
	{
		FControlRigTraitSharedData* ControlRigTraitSharedData = (FControlRigTraitSharedData*)ScopedControlRigTraitSharedData->GetStructMemory();
		TargetSkeleton = ControlRigTraitSharedData->GetPreviewSkeleton();
	}
	else if (ScopedControlRigTraitSharedData->GetStruct() == FControlRigTraitSharedData_v2::StaticStruct())
	{
		FControlRigTraitSharedData_v2* ControlRigTraitSharedData = (FControlRigTraitSharedData_v2*)ScopedControlRigTraitSharedData->GetStructMemory();
		TargetSkeleton = ControlRigTraitSharedData->GetPreviewSkeleton();
	}

	return TargetSkeleton;
}

const FControlRigIOMapping::FControlsInfo* FControlRigTraitSharedDataCustomization::GetControlInfo(const TArray<FControlRigIOMapping::FControlsInfo>& Controls, const FName& ControlName)
{
	const FControlRigIOMapping::FControlsInfo* ControlInfo = Controls.FindByPredicate([&ControlName](const FControlRigIOMapping::FControlsInfo& In)
		{
			return In.Name == ControlName;
		});

	return ControlInfo;
}

template<typename TSharedData>
bool FControlRigTraitSharedDataCustomization::IsVariableProperty(TSharedData* ControlRigTraitSharedData, const FName& PropertyName )
{
	if (ControlRigTraitSharedData != nullptr)
	{
		FControlRigAssetStrongReference ControlRigClass = ControlRigTraitSharedData->GetControlRigAssetReference();
		if (ControlRigClass.IsValid())
		{
			const TArray<FRigVMExternalVariable> PublicVariables = ControlRigClass.GetExternalVariables();

			const FRigVMExternalVariable* PublicVar = PublicVariables.FindByPredicate([&PropertyName](const FRigVMExternalVariable& In)
				{
					return In.GetName() == PropertyName;
				});

			return PublicVar != nullptr;
		}
	}

	return false;
}

void FControlRigTraitSharedDataCustomization::OnObjectsReinstanced(const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	if (ScopedControlRigTraitSharedData.IsValid())
	{
		for (const TPair<UObject*, UObject*>& Pair : OldToNewInstanceMap)
		{
			UObject* NewObject = Pair.Value;
			if ((NewObject != nullptr) && NewObject->IsA<UControlRig>())
			{
				FControlRigAssetStrongReference ControlRigClass;
				if (ScopedControlRigTraitSharedData->GetStruct() == FControlRigTraitSharedData::StaticStruct())
				{
					FControlRigTraitSharedData* ControlRigTraitSharedData = (FControlRigTraitSharedData*)ScopedControlRigTraitSharedData->GetStructMemory();
					ControlRigClass = ControlRigTraitSharedData->GetControlRigAssetReference();
				}
				else if (ScopedControlRigTraitSharedData->GetStruct() == FControlRigTraitSharedData_v2::StaticStruct())
				{
					FControlRigTraitSharedData_v2* ControlRigTraitSharedData = (FControlRigTraitSharedData_v2*)ScopedControlRigTraitSharedData->GetStructMemory();
					ControlRigClass = ControlRigTraitSharedData->GetControlRigAssetReference();
				}
				if (FControlRigTraitSharedDataBase* ControlRigTraitSharedData = (FControlRigTraitSharedDataBase*)ScopedControlRigTraitSharedData->GetStructMemory())
				{
					if (ControlRigClass.IsValid())
					{
						if (URigVMEdGraphNode* EdGraphNode = Cast<URigVMEdGraphNode>(SelectedNodeWeak.Get()))
						{
							bool bRequiresPinRefresh = false;

							const TArray<FRigVMExternalVariable> PublicVariables = ControlRigClass.GetExternalVariables();

							// Check if the currently exposed properties still exist
							for (const FOptionalPinFromProperty& PinProperty : CustomPinProperties)
							{
								if (!PinProperty.bShowPin) // if not exposed, no need to test
								{
									continue;
								}

								const FName& PropertyName = PinProperty.PropertyName;
								const FRigVMExternalVariable* PublicVar = PublicVariables.FindByPredicate([&PropertyName](const FRigVMExternalVariable& In)
									{
										return In.GetName() == PropertyName;
									});
						
								if (PublicVar != nullptr) // found a public var with the same name of the exposed pin
								{
									const TArray<URigVMPin*>& RootPins = EdGraphNode->GetInputPins();
									for (const URigVMPin* RootPîn : RootPins)
									{
										// Try to find a node pin with the same name
										if (const URigVMPin*const* PinPtr = RootPîn->GetSubPins().FindByPredicate([&PropertyName](const URigVMPin* In)
											{
												return In->GetFName() == PropertyName;
											}))
										{
											const URigVMPin* Pin = (*PinPtr);
											if (Pin->GetCPPType() != PublicVar->GetExtendedCPPType())
											{
												bRequiresPinRefresh = true;
												break;
											}
										}
										else
										{
											// No pin found but if it is exposed, refresh
											if (ControlRigTraitSharedData->ExposedPropertyVariableNames.Contains(PropertyName) 
												|| ControlRigTraitSharedData->ExposedPropertyControlNames.Contains(PropertyName))
											{
												bRequiresPinRefresh = true;
												break;
											}
											// else all ok, no pin, but it is not exposed
										}
									}
								}
								else
								{
									// No public var found, maybe a control ?
									const TArray<FControlRigIOMapping::FControlsInfo>& Controls = ControlRigIOMapping->GetControls();
									if(Controls.FindByPredicate([&PropertyName](const FControlRigIOMapping::FControlsInfo& In)
										{
											return In.Name == PropertyName;
										}) == nullptr)
									{
										// No control found, request a rebuild
										bRequiresPinRefresh = true;
										break;
									}
								}
							}

							if (bRequiresPinRefresh)
							{
								if (URigVMController* Controller = EdGraphNode->GetController())
								{
									FRigVMControllerCompileBracketScope CompileScope(Controller); // avoid multiple recompilations and force a details refresh after recompile
									Controller->RepopulatePinsOnNode(EdGraphNode->GetModelNode(), true, false, true); // Force pin refresh on node
								}
							}
						}
					}
				}
				break;
			}
		}
	}
}

FString FControlRigTraitSharedDataCustomization::FindControlRigTraitPinName(URigVMNode* ModelNode)
{
	const FTraitRegistry& TraitRegistry = FTraitRegistry::Get();

	FString ControlRigTraitName;
	FString ControlRigTraitDisplayName;
	if (const FTrait* Trait = TraitRegistry.Find(FControlRigTrait::TraitUID)) // the context menu in the traits create the traits using a sanitized display name
	{
		ControlRigTraitName = Trait->GetTraitName();
		ControlRigTraitName.RemoveAt(0);
		FString DisplayNameMetadata;
		Trait->GetTraitSharedDataStruct()->GetStringMetaDataHierarchical(FRigVMStruct::DisplayNameMetaName, &DisplayNameMetadata);
		ControlRigTraitDisplayName = DisplayNameMetadata.IsEmpty() ? Trait->GetTraitName() : DisplayNameMetadata;
		RigVMStringUtils::SanitizeName(ControlRigTraitDisplayName, false, false, URigVMSchema::GetMaxNameLength());
	}
	else
	{
		ControlRigTraitName = FString(TEXT("ControlRigTrait"));
	}

	FString Tmp;
	for (const auto& Pin : ModelNode->GetPins())
	{
		Pin->GetName(Tmp);

		if (Tmp.Contains(ControlRigTraitName))
		{
			return Tmp;
		}

		if (Tmp.Contains(ControlRigTraitDisplayName)) 
		{
			return Tmp;
		}
	}

	return FString();
}

} // end namespace UE::UAF::Editor


#undef LOCTEXT_NAMESPACE
