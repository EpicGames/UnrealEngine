// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class UAFControlRigEditor : ModuleRules
	{
		public UAFControlRigEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"CoreUObject",
					"Engine",
					"Core",
					"ControlRig",
					"ControlRigDeveloper",
					"RigVM",
					"UAF",
					"UAFAnimGraph",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"UnrealEd",
					"RigVMDeveloper",
					"WorkspaceEditor",
					"SlateCore",
					"Slate",
					"RigVMEditor",
					"UAFEditor",
					"UAFControlRig",
					"UAFUncookedOnly",
					"UAFAnimGraphUncookedOnly",
				}
			);
		}
	}
}
